// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.IO;

namespace NetworkProfiler
{
    /**
     * Big endian version of binary reader.
     */
    public class BinaryReaderBigEndian : BinaryReader
    {
		byte[] SwappedBytes = new byte[8];

        /**
         * Constructor, passing arguments to base class.
         */
        public BinaryReaderBigEndian(Stream Input)
            : base(Input, System.Text.Encoding.ASCII)
        {
        }

        /**
         * Reads & swaps bytes array of size count.
         */
        private byte[] ReadSwappedBytes(int Count)
        {
            for (int i = Count - 1; i >= 0; i--)
            {
                SwappedBytes[i] = ReadByte();
            }
            return SwappedBytes;
        }

        /**
         * Reads an UInt16 from stream and returns it.
         */
        public override ushort ReadUInt16()
        {
            return BitConverter.ToUInt16(ReadSwappedBytes(2), 0);
        }

        /**
         * Reads an Int16 from stream and returns it.
         */
        public override short ReadInt16()
        {
            return BitConverter.ToInt16(ReadSwappedBytes(2), 0);
        }

        /**
         * Reads an UInt32 from stream and returns it.
         */
        public override int ReadInt32()
        {
            return BitConverter.ToInt32(ReadSwappedBytes(4), 0);
        }

        /**
         * Reads an Int32 from stream and returns it.
         */
        public override uint ReadUInt32()
        {
            return BitConverter.ToUInt32(ReadSwappedBytes(4), 0);
        }

        /**
         * Reads an UInt64 from stream and returns it.
         */
        public override long ReadInt64()
        {
            return BitConverter.ToInt64(ReadSwappedBytes(8), 0);
        }

        /**
         * Reads an Int64 from stream and returns it.
         */
        public override ulong ReadUInt64()
        {
            return BitConverter.ToUInt64(ReadSwappedBytes(8), 0);
        }

        /**
         * Reads a float from stream and returns it.
         */
        public override float ReadSingle()
        {
            return BitConverter.ToSingle(ReadSwappedBytes(4), 0);
        }
    }
}