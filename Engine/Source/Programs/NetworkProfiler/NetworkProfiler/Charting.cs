// Copyright Epic Games, Inc. All Rights Reserved.



using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Windows.Forms.DataVisualization;
using System.Windows.Forms.DataVisualization.Charting;
using System.Drawing;
using System.Windows.Forms;

// Microsoft Chart Controls Add-on for Microsoft Visual Studio 2008
//
// http://www.microsoft.com/downloads/details.aspx?familyid=1D69CE13-E1E5-4315-825C-F14D33A303E9&displaylang=en

namespace NetworkProfiler
{
	class ChartParser
	{
		public static void ParseStreamIntoChart( MainWindow InMainWindow, NetworkStream NetworkStream, Chart NetworkChart, FilterValues InFilterValues )
		{
			var StartTime = DateTime.UtcNow;

			InMainWindow.ShowProgress( true );

			// Save old scroll position
			double OldPosition = NetworkChart.ChartAreas["DefaultChartArea"].AxisX.ScaleView.Position;

			NetworkChart.BeginInit();

			// Reset existing data.
			for ( int i = 0; i < NetworkChart.Series.Count; i++ )
			{
				float Percent = ( float )i / ( float )NetworkChart.Series.Count;
				InMainWindow.UpdateProgress( ( int )( Percent * 100 ) );

				NetworkChart.Series[i].Points.Clear();
			}

			InMainWindow.ShowProgress( true );

			NetworkChart.ResetAutoValues();
			NetworkChart.Invalidate();

			NetworkChart.ChartAreas[0].AxisX.ScrollBar.IsPositionedInside = false;
			NetworkChart.ChartAreas[0].AxisX.ScrollBar.ButtonStyle = ScrollBarButtonStyles.All;
			NetworkChart.ChartAreas[0].AxisX.ScrollBar.Size = 15;
			NetworkChart.ChartAreas[0].AxisX.ScrollBar.ButtonColor = Color.LightGray;

			NetworkChart.ChartAreas[0].AxisY.ScrollBar.IsPositionedInside = false;
			NetworkChart.ChartAreas[0].AxisY.ScrollBar.ButtonStyle = ScrollBarButtonStyles.All;
			NetworkChart.ChartAreas[0].AxisY.ScrollBar.Size = 15;
			NetworkChart.ChartAreas[0].AxisY.ScrollBar.ButtonColor = Color.LightGray;

			int FrameCounter = 0;

			foreach( PartialNetworkStream RawFrame in NetworkStream.Frames )
			{
				if ( FrameCounter % 1000 == 0 )
				{
					float Percent = ( float )FrameCounter / ( float )NetworkStream.Frames.Count;
					InMainWindow.UpdateProgress( ( int )( Percent * 100 ) );
				}

				PartialNetworkStream Frame = RawFrame.Filter( InFilterValues );

				if( Frame.EndTime == Frame.StartTime )
				{
					throw new InvalidOperationException();
				}

				float OneOverDeltaTime = 1 / (Frame.EndTime - Frame.StartTime);

				int OutgoingBandwidth = Frame.UnrealSocketSize + Frame.OtherSocketSize + NetworkStream.PacketOverhead * ( Frame.UnrealSocketCount + Frame.OtherSocketCount );

				InMainWindow.AddChartPoint( SeriesType.OutgoingBandwidthSize,		FrameCounter, OutgoingBandwidth );
				InMainWindow.AddChartPoint( SeriesType.OutgoingBandwidthSizeSec,	FrameCounter, OutgoingBandwidth * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.ActorCount,					FrameCounter, Frame.ActorCount );
				InMainWindow.AddChartPoint( SeriesType.PropertySize,				FrameCounter, Frame.ReplicatedSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.PropertySizeSec,				FrameCounter, Frame.ReplicatedSizeBits / 8 * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.RPCSize,						FrameCounter, Frame.RPCSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.RPCSizeSec,					FrameCounter, Frame.RPCSizeBits / 8 * OneOverDeltaTime );

#if true
				InMainWindow.AddChartPoint( SeriesType.ActorCountSec,				FrameCounter, Frame.ActorCount * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.PropertyCount,				FrameCounter, Frame.PropertyCount );
				InMainWindow.AddChartPoint( SeriesType.PropertyCountSec,			FrameCounter, Frame.PropertyCount * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.RPCCount,					FrameCounter, Frame.RPCCount );
				InMainWindow.AddChartPoint( SeriesType.RPCCountSec,					FrameCounter, Frame.RPCCount * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.ExportBunchCount,			FrameCounter, Frame.ExportBunchCount );
				InMainWindow.AddChartPoint( SeriesType.ExportBunchSize,				FrameCounter, Frame.ExportBunchSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.MustBeMappedGuidsCount,		FrameCounter, Frame.MustBeMappedGuidCount / 8 );
				InMainWindow.AddChartPoint( SeriesType.MustBeMappedGuidsSize,		FrameCounter, Frame.MustBeMappedGuidSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.SendAckCount,				FrameCounter, Frame.SendAckCount );
				InMainWindow.AddChartPoint( SeriesType.SendAckCountSec,				FrameCounter, Frame.SendAckCount * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.SendAckSize,					FrameCounter, Frame.SendAckSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.SendAckSizeSec,				FrameCounter, Frame.SendAckSizeBits / 8 * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.ContentBlockHeaderSize,		FrameCounter, Frame.ContentBlockHeaderSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.ContentBlockFooterSize,		FrameCounter, Frame.ContentBlockFooterSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.PropertyHandleSize,			FrameCounter, Frame.PropertyHandleSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.SendBunchCount,				FrameCounter, Frame.SendBunchCount );
				InMainWindow.AddChartPoint( SeriesType.SendBunchCountSec,			FrameCounter, Frame.SendBunchCount * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.SendBunchSize,				FrameCounter, Frame.SendBunchSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.SendBunchSizeSec,			FrameCounter, Frame.SendBunchSizeBits / 8 * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.SendBunchHeaderSize,			FrameCounter, Frame.SendBunchHeaderSizeBits / 8 );
				InMainWindow.AddChartPoint( SeriesType.GameSocketSendSize,			FrameCounter, Frame.UnrealSocketSize );
				InMainWindow.AddChartPoint( SeriesType.GameSocketSendSizeSec,		FrameCounter, Frame.UnrealSocketSize * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.GameSocketSendCount,			FrameCounter, Frame.UnrealSocketCount );
				InMainWindow.AddChartPoint( SeriesType.GameSocketSendCountSec,		FrameCounter, Frame.UnrealSocketCount * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.ActorReplicateTimeInMS,		FrameCounter, Frame.ActorReplicateTimeInMS);
#endif

#if false
				InMainWindow.AddChartPoint( SeriesType.MiscSocketSendSize,			FrameCounter, Frame.OtherSocketSize );
				InMainWindow.AddChartPoint( SeriesType.MiscSocketSendSizeSec,		FrameCounter, Frame.OtherSocketSize * OneOverDeltaTime );
				InMainWindow.AddChartPoint( SeriesType.MiscSocketSendCount,			FrameCounter, Frame.OtherSocketCount );
				InMainWindow.AddChartPoint( SeriesType.MiscSocketSendCountSec,		FrameCounter, Frame.OtherSocketCount * OneOverDeltaTime );								
#endif

				if ( Frame.NumEvents > 0 )
				{
					InMainWindow.AddChartPoint( SeriesType.Events, FrameCounter, 0 );
				}

				FrameCounter++;
			}

            string OutgoingBandwidthSizeSecAsString = SeriesType.OutgoingBandwidthSizeSec.ToString();

            NetworkChart.DataManipulator.IsStartFromFirst = true;
            NetworkChart.DataManipulator.FinancialFormula(FinancialFormula.MovingAverage, Math.Min(30, NetworkChart.Series[OutgoingBandwidthSizeSecAsString].Points.Count).ToString(), OutgoingBandwidthSizeSecAsString, SeriesType.OutgoingBandwidthSizeAvgSec.ToString());

            NetworkChart.ChartAreas["DefaultChartArea"].RecalculateAxesScale();

			NetworkChart.ChartAreas["DefaultChartArea"].AxisX.ScaleView.Position = OldPosition;

			NetworkChart.EndInit();

			InMainWindow.ShowProgress( false );

            Console.WriteLine("Adding data to chart took {0} seconds", (DateTime.UtcNow - StartTime).TotalSeconds);
		}
	}
}
