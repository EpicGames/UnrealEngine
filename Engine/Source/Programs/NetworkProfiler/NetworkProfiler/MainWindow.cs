// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Text;
using System.Windows.Forms;
using System.Windows.Forms.DataVisualization.Charting;
using System.IO;
using System.Threading;
using System.Diagnostics;

namespace NetworkProfiler
{
	public enum SeriesType : int
	{
		OutgoingBandwidthSize = 0,
		OutgoingBandwidthSizeSec,
		OutgoingBandwidthSizeAvgSec,
		ActorCount,
		PropertySize,
		PropertySizeSec,
		RPCSize,
		RPCSizeSec,
		Events,
		ActorCountSec,
		PropertyCount,
		PropertyCountSec,
		RPCCount,
		RPCCountSec,
		ExportBunchCount,
		ExportBunchSize,
		MustBeMappedGuidsCount,
		MustBeMappedGuidsSize,
		SendAckCount,
		SendAckCountSec,
		SendAckSize,
		SendAckSizeSec,
		ContentBlockHeaderSize,
		ContentBlockFooterSize,
		PropertyHandleSize,
		SendBunchCount,
		SendBunchCountSec,
		SendBunchSize,
		SendBunchSizeSec,
		SendBunchHeaderSize,
		GameSocketSendSize,
		GameSocketSendSizeSec,
		GameSocketSendCount,
		GameSocketSendCountSec,
		ActorReplicateTimeInMS,
#if false
		MiscSocketSendSize,
		MiscSocketSendSizeSec,
		MiscSocketSendCount,
		MiscSocketSendCountSec,
#endif
	};

	public partial class MainWindow : Form
	{
		/** Currently selected frame on chart			*/
		int CurrentFrame			= 0;

		/** Marks last selected range on chart			*/
		int RangeSelectStart		= -1;
		int RangeSelectEnd			= -1;

		/** Current network stream.						*/
		NetworkStream CurrentNetworkStream = null;

		/** Current filter values						*/
		FilterValues CurrentFilterValues = new FilterValues();

		/** Worker threads								*/
		Thread LoadThread			= null;
		Thread SelectRangeThread	= null;

		PartialNetworkStream CurrentStreamSelection = null;

		/** If non 0, we will early out of loading in this many minutes worth of profile time */
		int MaxProfileMinutes		= 0;

		public MainWindow( string[] Args )
		{
			InitializeComponent();

			this.Closing += OnFormClosing;

			this.AllowDrop = true;
			this.DragEnter += OnDragEnter;
			this.DragDrop += OnDragDrop;

			// Set default state.
			SetDefaultLineView();

			// Force the columns to the way we want them
			SetupColumns( ActorListView,	new String[]	{ "Total Size (KBytes)", "Count", "Average Size (Bytes)", "Average Size (Bits)", "Time (ms)", "Average Time (ms)", "Actor Class" });
			SetupColumns( PropertyListView, new String[]	{ "Total Size (KBytes)", "Count", "Average Size (Bytes)", "Average Size (Bits)", "Time (ms)", "Average Time (ms)", "Property" });
			SetupColumns( RPCListView,		new String[]	{ "Total Size (KBytes)", "Count", "Average Size (Bytes)", "Average Size (Bits)", "Time (ms)", "Average Time (ms)", "RPC" });

			SetupColumns( ActorPerfPropsListView, new String[] { "Actor", "MS", "MS %", "KB/s", "Bytes", "Count", "Update HZ", "Rep HZ", "Waste" } );
			SetupColumns( ActorPerfPropsDetailsListView, new String[] { "Property", "Bytes", "Count" } );

			SetupColumns( ObjectReplicationListView, new String[] { "Object Class", "# Comparisons", "# Replications", "Comparison Time (ms)", "Avg. Time Per Compare (ms)" } );
			SetupColumns( ObjectPropertyReplicationListView, new String[] { "Property", "# Comparisons", "# Times Changed", "# Replications" } );

			ActorPerfPropsDetailsListView.Columns[0].Width = 170;
			ActorPerfPropsDetailsListView.Columns[1].Width = 50;
			ActorPerfPropsDetailsListView.Columns[2].Width = 50;

			if ( Args.Length > 0 )
			{
				ChangeNetworkStream( Args[0] );
			}
		}
		private void OnDragEnter( object sender, DragEventArgs e )
		{
			if ( e.Data.GetDataPresent( DataFormats.FileDrop ) )
			{
				e.Effect = DragDropEffects.Copy;
			}
		}

		private void OnDragDrop( object sender, DragEventArgs e )
		{
			string[] Files = ( string[] )e.Data.GetData( DataFormats.FileDrop );
			if( Files.Length > 0 )
			{
				ChangeNetworkStream( Files[0] );
			}
		}

		private void OnFormClosing( object sender, CancelEventArgs e )
		{
			CancelLoadThread();
			CancelSelectRangeThread();
		}

		void NetworkChart_PostPaint( object sender, ChartPaintEventArgs e )
		{
			if ( RangeSelectEnd - RangeSelectStart <= 0 )
			{
				return;
			}

			if ( e.ChartGraphics.Graphics != null && e.ChartElement is ChartArea && e.Chart.Series.Count > 0 && e.Chart.Series[0].Points.Count > 0 )
			{
				ChartGraphics cg = e.ChartGraphics;

				System.Drawing.PointF Pos1 = System.Drawing.PointF.Empty;				

				Pos1.X = ( float )cg.GetPositionFromAxis( "DefaultChartArea", AxisName.X, RangeSelectStart );
				Pos1.Y = ( float )cg.GetPositionFromAxis( "DefaultChartArea", AxisName.Y, ( float )NetworkChart.ChartAreas["DefaultChartArea"].AxisY.Minimum );

				System.Drawing.PointF Pos2 = System.Drawing.PointF.Empty;

				Pos2.X = ( float )cg.GetPositionFromAxis( "DefaultChartArea", AxisName.X, RangeSelectEnd );
				Pos2.Y = ( float )cg.GetPositionFromAxis( "DefaultChartArea", AxisName.Y, NetworkChart.ChartAreas["DefaultChartArea"].AxisY.Maximum );

				// Convert relative coordinates to absolute coordinates.
				Pos1 = cg.GetAbsolutePoint( Pos1 );
				Pos2 = cg.GetAbsolutePoint( Pos2 );

				SolidBrush Brush = new SolidBrush( Color.FromArgb( 20, 255, 0, 0 ) );

				cg.Graphics.FillRectangle
				(
				   Brush,
				   Pos1.X,
				   Pos2.Y,
				   Pos2.X - Pos1.X, 
				   Pos1.Y - Pos2.Y 
				);
			}
		}

		private void SetDefaultLineView()
		{
			// Set default state.
			NetworkChart.Series.Clear();
			ChartListBox.Items.Clear();

			RegisterChartSeries( SeriesType.OutgoingBandwidthSize,		"Outgoing Bandwidth Bytes",		false );
			RegisterChartSeries( SeriesType.OutgoingBandwidthSizeSec,	"Outgoing Bandwidth Bytes/s",	true );
			RegisterChartSeries( SeriesType.OutgoingBandwidthSizeAvgSec,"Outgoing Bandwidth Avg/s",		true );
			RegisterChartSeries( SeriesType.ActorCount,					"Actor Count", false );
			RegisterChartSeries( SeriesType.PropertySize,				"Property Bytes",				false );
			RegisterChartSeries( SeriesType.PropertySizeSec,			"Property Bytes/s",				true );
			RegisterChartSeries( SeriesType.RPCSize,					"RPC Bytes",					false );
			RegisterChartSeries( SeriesType.RPCSizeSec,					"RPC Bytes/s",					true );
			RegisterChartSeries( SeriesType.Events,						"Events",						false );

			RegisterChartSeries( SeriesType.ActorCountSec,				"Actor Count/s",				false );
			RegisterChartSeries( SeriesType.PropertyCount,				"Property Count",				false );
			RegisterChartSeries( SeriesType.PropertyCountSec,			"Property Count/s",				false );
			RegisterChartSeries( SeriesType.RPCCount,					"RPC Count",					false );
			RegisterChartSeries( SeriesType.RPCCountSec,				"RPC Count/s",					false );
			RegisterChartSeries( SeriesType.ExportBunchCount,			"Export Bunch Count",			false );
			RegisterChartSeries( SeriesType.ExportBunchSize,			"Export Bunch Count/s",			false );
			RegisterChartSeries( SeriesType.MustBeMappedGuidsCount,		"Must Be Mapped Guid Count",	false );
			RegisterChartSeries( SeriesType.MustBeMappedGuidsSize,		"Must Be Mapped Guid Bytes",	false );
			RegisterChartSeries( SeriesType.SendAckCount,				"Send Ack Count",				false );
			RegisterChartSeries( SeriesType.SendAckCountSec,			"Send Ack Count/s",				false );
			RegisterChartSeries( SeriesType.SendAckSize,				"Send Ack Bytes",				false );
			RegisterChartSeries( SeriesType.SendAckSizeSec,				"Send Ack Bytes/s",				false );
			RegisterChartSeries( SeriesType.ContentBlockHeaderSize,		"Content Block Header Bytes",	false );
			RegisterChartSeries( SeriesType.ContentBlockFooterSize,		"Content Block Footer Bytes",	false );
			RegisterChartSeries( SeriesType.PropertyHandleSize,			"Property Handle Bytes",		false );
			RegisterChartSeries( SeriesType.SendBunchCount,				"Send Bunch Count",				false );
			RegisterChartSeries( SeriesType.SendBunchCountSec,			"Send Bunch Count/s",			false );
			RegisterChartSeries( SeriesType.SendBunchSize,				"Send Bunch Bytes",				false );
			RegisterChartSeries( SeriesType.SendBunchSizeSec,			"Send Bunch Bytes/s",			false );
			RegisterChartSeries( SeriesType.SendBunchHeaderSize,		"Send Bunch Header Bytes",		false );
			RegisterChartSeries( SeriesType.GameSocketSendSize,			"Game Socket Send Bytes",		false );
			RegisterChartSeries( SeriesType.GameSocketSendSizeSec,		"Game Socket Send Bytes/s",		false );
			RegisterChartSeries( SeriesType.GameSocketSendCount,		"Game Socket Send Count",		false );
			RegisterChartSeries( SeriesType.GameSocketSendCountSec,		"Game Socket Send Count/s",		false );
			RegisterChartSeries( SeriesType.ActorReplicateTimeInMS,		"Actor Replicate Time In MS",	false );

#if false
			RegisterChartSeries( SeriesType.MiscSocketSendSize,			"Misc Socket Send Bytes",		false );
			RegisterChartSeries( SeriesType.MiscSocketSendSizeSec,		"Misc Socket Send Bytes/s",		false );
			RegisterChartSeries( SeriesType.MiscSocketSendCount,		"Misc Socket Send Count",		false );
			RegisterChartSeries( SeriesType.MiscSocketSendCountSec,		"Misc Socket Send Count/s",		false );
#endif

			NetworkChart.Series["Events"].ChartType = SeriesChartType.Point;

			ConnectionListBox.Items.Clear();

			ShowProgress( false );
		}

		protected void RegisterChartSeries( SeriesType Type, string FriendlyName, bool bEnabled )
		{
			NetworkChart.Series.Add( new Series( Type.ToString() ) );

			Debug.Assert( NetworkChart.Series[( int )Type].Name == Type.ToString() );

			NetworkChart.Series[( int )Type].XValueType = ChartValueType.Int32;
			NetworkChart.Series[( int )Type].Font = new System.Drawing.Font( "Tahoma", 8.25F, System.Drawing.FontStyle.Regular, System.Drawing.GraphicsUnit.Point, ( ( byte )( 0 ) ) );
			NetworkChart.Series[( int )Type].Legend = "DefaultLegend";
			NetworkChart.Series[( int )Type].ChartArea = "DefaultChartArea";

			ChartListBox.Items.Add( FriendlyName );

			Debug.Assert( ChartListBox.Items.Count - 1 == ( int )Type );

			UpdateChartSeries( Type, SeriesChartType.FastLine, bEnabled );
		}

		protected void UpdateChartSeries( SeriesType Type, SeriesChartType ChartType, bool bEnabled )
		{
			NetworkChart.Series[( int )Type].ChartType = ChartType;
			NetworkChart.Series[( int )Type].Enabled = bEnabled;
			ChartListBox.SetItemChecked( ( int )Type, bEnabled );
		}

		public void AddChartPoint( SeriesType Type, double X, double Y )
		{
			NetworkChart.Series[ (int)Type ].Points.AddXY( X, Y );
		}

		public void UpdateProgress( Int32 Value )
		{
			if ( this.InvokeRequired )
			{
				this.Invoke( new Action( () => UpdateProgress( Value ) ) );
				return;
			}

			CurrentProgress.Value = Math.Max( 0, Math.Min( 100, Value ) );
		}

		public void ShowProgress( bool bShow )
		{
			if ( this.InvokeRequired )
			{
				this.Invoke( new Action( () => ShowProgress( bShow ) ) );
				return;
			}

			CurrentProgress.Visible = bShow;
			CurrentProgress.Value = 0;
		}

		private void SetDefaultStackedBunchSizeView()
		{
			foreach( var Series in NetworkChart.Series )
			{
				Series.Enabled = false;
			}

			for( int i = 0; i < ChartListBox.Items.Count; ++i )
			{
				ChartListBox.SetItemChecked(i, false);
			}

			UpdateChartSeries( SeriesType.SendBunchSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.PropertySize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.RPCSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.ExportBunchSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.MustBeMappedGuidsSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.SendBunchHeaderSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.ContentBlockHeaderSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.ContentBlockFooterSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.PropertyHandleSize, SeriesChartType.StackedArea, true );
		}

		private void SetDefaultFastLineView()
		{
			foreach ( var Series in NetworkChart.Series )
			{
				Series.Enabled = false;
			}

			for ( int i = 0; i < ChartListBox.Items.Count; ++i )
			{
				ChartListBox.SetItemChecked( i, false );
			}

			UpdateChartSeries( SeriesType.SendBunchSize, SeriesChartType.FastLine, true );
			UpdateChartSeries( SeriesType.PropertySize, SeriesChartType.FastLine, true );
			UpdateChartSeries( SeriesType.RPCSize, SeriesChartType.FastLine, true );
			UpdateChartSeries( SeriesType.ExportBunchSize, SeriesChartType.FastLine, true );
			UpdateChartSeries( SeriesType.MustBeMappedGuidsSize, SeriesChartType.FastLine, true );
			UpdateChartSeries( SeriesType.SendBunchHeaderSize, SeriesChartType.StackedArea, true );
			UpdateChartSeries( SeriesType.ContentBlockHeaderSize, SeriesChartType.FastLine, true );
			UpdateChartSeries( SeriesType.ContentBlockFooterSize, SeriesChartType.FastLine, true );
			UpdateChartSeries( SeriesType.PropertyHandleSize, SeriesChartType.FastLine, true );
		}

		private void SetupColumns(ListView ListView, String[] Headers)
		{
			ListView.Columns.Clear();
			foreach (var Header in Headers)
			{
				AddColumn(ListView, Header);
			}

			ListView.AutoResizeColumns(ColumnHeaderAutoResizeStyle.ColumnContent);
			ListView.AutoResizeColumns(ColumnHeaderAutoResizeStyle.HeaderSize);
		}

		private void AddColumn( ListView ListView, String HeaderText )
		{
			ColumnHeader header = new ColumnHeader();
			header.Text = HeaderText;
			ListView.Columns.Add(header);
		}

		private void ChangeNetworkStreamWorker( string Filename )
		{
			using ( var ParserStream = File.OpenRead( Filename ) )
			{
				try
				{
					CurrentNetworkStream = StreamParser.Parse( this, ParserStream );
					ParseStreamForListViews();
					ChartParser.ParseStreamIntoChart( this, CurrentNetworkStream, NetworkChart, CurrentFilterValues );
				}
				catch ( System.Threading.ThreadAbortException )
				{

				}
				catch ( System.Exception se)
				{
					Console.Out.WriteLine(se.StackTrace);
					ClearStreamAndChart();
				}
			}

			LoadThread = null;
		}

		private void CancelLoadThread()
		{
			if ( LoadThread != null )
			{
				LoadThread.Abort();
				LoadThread = null;
			}
		}

		private void ChangeNetworkStream( string Filename )
		{
			CancelLoadThread();

			LoadThread = new Thread( () => ChangeNetworkStreamWorker( Filename ) );
			LoadThread.Start();
		}

		public void ClearStreamAndChart()
		{
			if ( this.InvokeRequired )
			{
				this.Invoke( new Action( () => ClearStreamAndChart() ) );
				return;
			}

			CurrentNetworkStream = null;

			foreach ( var Series in NetworkChart.Series )
			{
				Series.Points.Clear();
			}
		}

		public void ParseStreamForListViews()
		{
			if ( this.InvokeRequired )
			{
				this.Invoke( new Action( () => ParseStreamForListViews() ) );
				return;
			}

			StreamParser.ParseStreamIntoListView( CurrentNetworkStream, CurrentNetworkStream.ActorNameToSummary, ActorListView );
			StreamParser.ParseStreamIntoListView( CurrentNetworkStream, CurrentNetworkStream.PropertyNameToSummary, PropertyListView );
			StreamParser.ParseStreamIntoListView( CurrentNetworkStream, CurrentNetworkStream.RPCNameToSummary, RPCListView );
			StreamParser.ParseStreamIntoReplicationListView( CurrentNetworkStream, CurrentNetworkStream.ObjectNameToReplicationSummary, ObjectReplicationListView );

			ActorFilterBox.Items.Clear();
			ActorFilterBox.Items.Add( "" );

			PropertyFilterBox.Items.Clear();
			PropertyFilterBox.Items.Add( "" );

			RPCFilterBox.Items.Clear();
			RPCFilterBox.Items.Add( "" );

			foreach ( var SummaryEntry in CurrentNetworkStream.ActorNameToSummary )
			{
				ActorFilterBox.Items.Add( CurrentNetworkStream.GetName( SummaryEntry.Key ) );
			}

			foreach ( var SummaryEntry in CurrentNetworkStream.PropertyNameToSummary )
			{
				PropertyFilterBox.Items.Add( CurrentNetworkStream.GetName( SummaryEntry.Key ) );
			}

			foreach ( var SummaryEntry in CurrentNetworkStream.RPCNameToSummary )
			{
				RPCFilterBox.Items.Add( CurrentNetworkStream.GetName( SummaryEntry.Key ) );
			}

			ConnectionListBox.Items.Clear();

			int NumberOfAddresses = ( CurrentNetworkStream.GetVersion() < 12 ) ? 
				CurrentNetworkStream.AddressArray.Count : CurrentNetworkStream.StringAddressArray.Count;

			for (int i = 0; i < NumberOfAddresses; i++)
			{
				ConnectionListBox.Items.Add( CurrentNetworkStream.GetIpString( i, CurrentNetworkStream.GetVersion() ) );
			}


			for ( int i = 0; i < ConnectionListBox.Items.Count; ++i )
			{
				ConnectionListBox.SetItemChecked( i, true );
			}
		}

		private void OpenButton_Click(object sender, EventArgs e)
		{			
			// Create a file open dialog for selecting the .nprof file.
			OpenFileDialog OpenFileDialog = new OpenFileDialog();
			OpenFileDialog.Title = "Open the profile data file from the game's 'Profiling' folder";
			OpenFileDialog.Filter = "Profiling Data (*.nprof)|*.nprof";
			OpenFileDialog.RestoreDirectory = false;
			OpenFileDialog.SupportMultiDottedExtensions = true;
			// Parse it if user didn't cancel.
			if( OpenFileDialog.ShowDialog() == DialogResult.OK )
			{
				// Create binary reader and file info object from filename.
				ChangeNetworkStream( OpenFileDialog.FileName );
			}			
		}

		private void NetworkChart_MouseClick(object sender, MouseEventArgs e)
		{
			// Toggle which axis to select and zoom on right click.
			if( e.Button == MouseButtons.Right )
			{
				bool Temp = NetworkChart.ChartAreas["DefaultChartArea"].CursorX.IsUserEnabled;				 
				NetworkChart.ChartAreas["DefaultChartArea"].CursorX.IsUserEnabled = NetworkChart.ChartAreas["DefaultChartArea"].CursorY.IsUserEnabled;
				NetworkChart.ChartAreas["DefaultChartArea"].CursorY.IsUserEnabled = Temp;

				Temp = NetworkChart.ChartAreas["DefaultChartArea"].CursorX.IsUserSelectionEnabled;				 
				NetworkChart.ChartAreas["DefaultChartArea"].CursorX.IsUserSelectionEnabled = NetworkChart.ChartAreas["DefaultChartArea"].CursorY.IsUserSelectionEnabled;
				NetworkChart.ChartAreas["DefaultChartArea"].CursorY.IsUserSelectionEnabled = Temp;
			}
		}

		private void ChartListBox_SelectedValueChanged( object sender, EventArgs e )
		{			
			// Save old scroll position
			double OldPosition = NetworkChart.ChartAreas["DefaultChartArea"].AxisX.ScaleView.Position;

			for ( int i=0; i < ChartListBox.Items.Count; i++ )
			{
				NetworkChart.Series[i].Enabled = ChartListBox.GetItemChecked( i );
			}
			// Reset scale based on visible graphs.
			NetworkChart.ChartAreas["DefaultChartArea"].RecalculateAxesScale();
			NetworkChart.ResetAutoValues();

			// Restore scroll position
			NetworkChart.ChartAreas["DefaultChartArea"].AxisX.ScaleView.Position = OldPosition;
		}

		/**
		 * Single clicking in the graph will change the summary to be the current frame.
		 */
		private void NetworkChart_CursorPositionChanged(object sender, System.Windows.Forms.DataVisualization.Charting.CursorEventArgs e)
		{
			CurrentFrame = (int) e.NewPosition;

			if( CurrentNetworkStream != null 
			&&	CurrentFrame >= 0 
			&&	CurrentFrame < CurrentNetworkStream.Frames.Count )
			{
				// Cancel range select so we stop highlighting that on the chart (there will be the red line now showing the single frame selection)
				RangeSelectStart	= -1;
				RangeSelectEnd		= -1;

				// Force chart repaint
				NetworkChart.Invalidate();

				SetCurrentStreamSelection( CurrentNetworkStream, CurrentNetworkStream.Frames[CurrentFrame].Filter( CurrentFilterValues ), true );
			}
		}

		public void SetCurrentStreamSelection( NetworkStream NetworkStream, PartialNetworkStream Selection, bool bSingleSelect )
		{
			if ( this.InvokeRequired )
			{
				this.Invoke( new Action( () => SetCurrentStreamSelection( NetworkStream, Selection, bSingleSelect ) ) );
				return;
			}

			ActorPerfPropsDetailsListView.Items.Clear();

			Selection.ToActorSummaryView( NetworkStream, ActorSummaryView );
			Selection.ToActorPerformanceView( NetworkStream, ActorPerfPropsListView, ActorPerfPropsDetailsListView, CurrentFilterValues );
			
			// Below is way too slow for range select right now, so we just do this for single frame selection
			if ( bSingleSelect )
			{
				Selection.ToDetailedTreeView( TokenDetailsView.Nodes, CurrentFilterValues );
			}

			CurrentStreamSelection = Selection;
		}

		private void SelectRangeWorker( int SelectionStart, int SelectionEnd )
		{
			// Create a partial network stream with the new selection to get the summary.
			PartialNetworkStream Selection = new PartialNetworkStream( 
													this,
													CurrentNetworkStream.Frames, 
													SelectionStart, 
													SelectionEnd,
													CurrentNetworkStream.NameIndexUnreal,
													CurrentFilterValues,
													1 / 30.0f
												);

			SetCurrentStreamSelection( CurrentNetworkStream, Selection, false );

			SelectRangeThread = null;
		}

		private void CancelSelectRangeThread()
		{
			if ( SelectRangeThread != null )
			{
				SelectRangeThread.Abort();
				SelectRangeThread = null;
			}
		}

		/**
		 * Selection dragging on the X axis will update the summary to be current selection.
		 */ 
		private void NetworkChart_SelectionRangeChanged(object sender, CursorEventArgs e)
		{
			if ( ( CurrentNetworkStream == null ) || ( CurrentNetworkStream.Frames.Count == 0 ) )
			{
				return;
			}

			if ( e.NewSelectionEnd - e.NewSelectionStart < 1 )
			{
				return;	// Single click, ignore, handled above
			}

			if( e.Axis.AxisName == AxisName.X )
			{
				ActorPerfPropsDetailsListView.Items.Clear();

				RangeSelectStart	= Math.Max( 0, ( int )NetworkChart.ChartAreas["DefaultChartArea"].AxisX.ScaleView.ViewMinimum );
				RangeSelectEnd		= Math.Min( CurrentNetworkStream.Frames.Count, ( int )NetworkChart.ChartAreas["DefaultChartArea"].AxisX.ScaleView.ViewMaximum );

				CancelSelectRangeThread();

				SelectRangeThread = new Thread( () => SelectRangeWorker( RangeSelectStart, RangeSelectEnd ) );
				SelectRangeThread.Start();
			}
		}

		private void ReloadChartWorker()
		{
			ChartParser.ParseStreamIntoChart( this, CurrentNetworkStream, NetworkChart, CurrentFilterValues );

			CancelSelectRangeThread();

			if ( RangeSelectEnd - RangeSelectStart > 0 )
			{
				SelectRangeThread = new Thread( () => SelectRangeWorker( RangeSelectStart, RangeSelectEnd ) );
			}
			else
			{
				SelectRangeThread = new Thread( () => SelectRangeWorker( 0, CurrentNetworkStream.Frames.Count ) );
			}

			SelectRangeThread.Start();

			LoadThread = null;
		}

		private void ApplyFiltersButton_Click(object sender, EventArgs e)
		{
			CurrentFilterValues.ActorFilter		= ActorFilterBox.Text		!= null	? ActorFilterBox.Text		: "";
			CurrentFilterValues.PropertyFilter	= PropertyFilterBox.Text	!= null	? PropertyFilterBox.Text	: "";
			CurrentFilterValues.RPCFilter		= RPCFilterBox.Text			!= null	? RPCFilterBox.Text			: "";

			UpdateConnectionFilter();

			CancelLoadThread();

			LoadThread = new Thread( () => ReloadChartWorker() );
			LoadThread.Start();
		}

		/**
		 * Comparator used to sort the columns of a list view in a specified order. Internally
		 * falls back to using the string comparator for its sorting if it can't convert text to number.
		 */
		private class ListViewItemComparer : System.Collections.IComparer 
		{
			/** Column to sort by		*/
			public int			SortColumn;
			/** Sort order for column	*/
			public SortOrder	SortOrder;
		
			/**
			 * Constructor
			 * 
			 * @param	InSortColumn	Column to sort by
			 * @param	InSortOrder		Sort order to use (either ascending or descending)	
			 */
			public ListViewItemComparer(int InSortColumn, SortOrder InSortOrder) 
			{
				SortColumn = InSortColumn;
				SortOrder = InSortOrder;
			}
			
			/**
			 * Compare function
			 */
			public int Compare( object A, object B ) 
			{
				string StringA = ((ListViewItem) A).SubItems[SortColumn].Text;
				string StringB = ((ListViewItem) B).SubItems[SortColumn].Text;
				int SortValue = 0;

				// Try converting to number, if that fails fall back to text compare
				double DoubleA = 0;
				double DoubleB = 0;
				if( Double.TryParse( StringA, out DoubleA ) && Double.TryParse( StringB, out DoubleB ) )
				{
					SortValue = Math.Sign( DoubleA - DoubleB );
				}
				else
				{
					SortValue = String.Compare( StringA, StringB );	
				}

				if( SortOrder == SortOrder.Descending )
				{
					return -SortValue;
				}
				else
				{
					return SortValue;
				}
			}
		}

		/**
		 * Helper function for dealing with list view column sorting
		 * 
		 * @param	RequestedSortColumn		Column to sort
		 * @param	ListView				ListView to sort
		 */
		private void HandleListViewSorting( int RequestedSortColumn, ListView ListView )
		{
			if( ListView.ListViewItemSorter != null )
			{
				var Comparer = ListView.ListViewItemSorter as ListViewItemComparer;
				// Change sort order if we're clicking on already sorted column.
				if( Comparer.SortColumn == RequestedSortColumn )
				{
					Comparer.SortOrder = Comparer.SortOrder == SortOrder.Descending ? SortOrder.Ascending : SortOrder.Descending;
				}
				// Sort by requested column, descending by default
				else
				{
					Comparer.SortColumn = RequestedSortColumn;
					Comparer.SortOrder = SortOrder.Descending;
				}
			}
			else
			{
				ListView.ListViewItemSorter = new ListViewItemComparer( RequestedSortColumn, SortOrder.Descending );
			}

			ListView.Sort();
		}


		private void PropertyListView_ColumnClick(object sender, ColumnClickEventArgs e)
		{
			HandleListViewSorting( e.Column, PropertyListView );
		}

		private void RPCListView_ColumnClick(object sender, ColumnClickEventArgs e)
		{
			HandleListViewSorting( e.Column, RPCListView );
		}

		private void ActorListView_ColumnClick(object sender, ColumnClickEventArgs e)
		{
			HandleListViewSorting( e.Column, ActorListView );
		}

		private void ActorPerfPropsListView_ColumnClick( object sender, ColumnClickEventArgs e )
		{
			HandleListViewSorting( e.Column, ActorPerfPropsListView );
		}

		private void ActorPerfPropsDetailsListView_ColumnClick( object sender, ColumnClickEventArgs e )
		{
			HandleListViewSorting( e.Column, ActorPerfPropsDetailsListView );
		}

		private void ActorPerfPropsListView_SelectedIndexChanged( object sender, EventArgs e )
		{
			ActorPerfPropsDetailsListView.Items.Clear();

			if ( LoadThread != null || SelectRangeThread != null || CurrentStreamSelection == null || ActorPerfPropsListView.SelectedItems.Count == 0 )
			{
				return;
			}

			CurrentStreamSelection.ToPropertyPerformanceView( CurrentNetworkStream, ActorPerfPropsListView.SelectedItems[0].Text, ActorPerfPropsDetailsListView, CurrentFilterValues );
		}

		private void LineViewRadioButton_CheckChanged( object sender, EventArgs e )
		{
			if (LineViewRadioButton.Checked)
			{
				SetDefaultFastLineView();
			}
		}

		private void StackedBunchSizeRadioButton_CheckChanged(object sender, EventArgs e)
		{
			if (StackedBunchSizeRadioButton.Checked)
			{
				SetDefaultStackedBunchSizeView();
			}
		}

		private void UpdateConnectionFilter()
		{
			CurrentFilterValues.ConnectionMask = new HashSet<int>();

			for ( int i = 0; i < ConnectionListBox.Items.Count; i++ )
			{
				if ( ConnectionListBox.GetItemChecked( i ) )
				{
					CurrentFilterValues.ConnectionMask.Add( i );
				}
			}
		}

		private void ConnectionListBox_SelectedValueChanged( object sender, EventArgs e )
		{
			UpdateConnectionFilter();
		}

		public int GetMaxProfileMinutes()
		{
			return MaxProfileMinutes;
		}

		private void MaxProfileMinutesTextBox_TextChanged( object sender, EventArgs e )
		{
			if ( MaxProfileMinutesTextBox.Text == "" )
			{
				MaxProfileMinutes = 0;
			}
			else try
			{
				MaxProfileMinutes = Int32.Parse( MaxProfileMinutesTextBox.Text );
			}
			catch
			{
				MaxProfileMinutes = 0;
				MaxProfileMinutesTextBox.Text = "";
			}
		}

        private void CheckAllConnectionsCheckBox_Click(object sender, System.EventArgs e)
        {
            bool bAreAllUnchecked = true;
            for (int i = 0; i < ConnectionListBox.Items.Count; i++)
            {
                if (ConnectionListBox.GetItemChecked(i) == false)
                {
                    bAreAllUnchecked = false;
                    break;
                }
            }

            for (int i = 0; i < ConnectionListBox.Items.Count; i++)
            {
                ConnectionListBox.SetItemChecked(i, !bAreAllUnchecked);
            }

            checkAllConnectionsCheckBox.Checked = !bAreAllUnchecked;
        }

		private void ObjectReplicationListView_SelectedIndexChanged(object sender, EventArgs e)
		{
			ReadOnlyCollection<PropertyReplicationSummary> Summaries = null;

			if (ObjectReplicationListView.SelectedItems.Count > 0)
			{
				string ObjectName = ObjectReplicationListView.SelectedItems[0].Text;
				int ObjectNameIndex = CurrentNetworkStream.NameArray.FindIndex(MaybeName => (ObjectName == MaybeName));

				if (ObjectNameIndex != -1)
				{
					Summaries = CurrentNetworkStream.ObjectNameToReplicationSummary[ObjectNameIndex].Properties;
				}
			}

			StreamParser.ParseStreamIntoPropertyReplicationListView(CurrentNetworkStream, Summaries, ObjectPropertyReplicationListView);
		}

		private void ObjectReplicationListView_ColumnClick(object sender, ColumnClickEventArgs e)
		{
			HandleListViewSorting(e.Column, ObjectReplicationListView);
		}

		private void ObjectPropertyReplicationListView_ColumnClick(object sender, ColumnClickEventArgs e)
		{
			HandleListViewSorting(e.Column, ObjectPropertyReplicationListView);
		}
	}

	public class FilterValues
	{
		public string ActorFilter		= "";
		public string PropertyFilter	= "";
		public string RPCFilter			= "";

		public HashSet<int> ConnectionMask = null;
	}
}
