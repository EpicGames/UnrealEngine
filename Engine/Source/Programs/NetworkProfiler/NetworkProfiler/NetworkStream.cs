// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.Linq;
using System.Text;
using System.Windows.Forms;

namespace NetworkProfiler
{
	/**
	 * Encapsulates entire network stream, split into frames. Also contains name table
	 * used to convert indices back into strings.
	 */
	public class NetworkStream
	{
		/** Per packet overhead to take into account for total outgoing bandwidth. */
		//public static int PacketOverhead = 48;
		public static int PacketOverhead = 28;

		/** Array of unique names. Code has fixed indexes into it.					*/
		public List<string> NameArray = new List<string>();

		/** Array of unique addresses. Code has fixed indexes into it.				*/
		public List<UInt64> AddressArray = new List<UInt64>();
		// Used for new storage method.
		public List<string> StringAddressArray = new List<string>();

		/** Last address index parsed from token stream								*/
		public int CurrentConnectionIndex = 0;

		/** Internal dictionary from class name to index in name array, used by GetClassNameIndex. */
		private Dictionary<string,int> ClassNameToNameIndex = new Dictionary<string,int>();
		
		/** Index of "Unreal" name in name array.									*/
		public int NameIndexUnreal = -1;

		/** At the highest level, the entire stream is a series of frames.			*/
		public List<PartialNetworkStream> Frames = new List<PartialNetworkStream>();

		/** Mapping from property name to summary */
		public Dictionary<int,TypeSummary> PropertyNameToSummary = new Dictionary<int,TypeSummary>();
		/** Mapping from actor name to summary */
		public Dictionary<int,TypeSummary> ActorNameToSummary = new Dictionary<int,TypeSummary>();
		/** Mapping from RPC name to summary */
		public Dictionary<int,TypeSummary> RPCNameToSummary = new Dictionary<int,TypeSummary>();

		public Dictionary<int, ObjectReplicationSummary> ObjectNameToReplicationSummary = new Dictionary<int, ObjectReplicationSummary>();

		public StreamHeader Header;

		/**
		 * Returns the name associated with the passed in index.
		 * 
		 * @param	Index	Index in name table
		 * @return	Name associated with name table index
		 */
		public string GetName(int Index)
		{
			return NameArray[Index];
		}

		/**
		 * Returns the ip address string associated with the passed in connection index.
		 * 
		 * @param	ConnectionIndex	Index in address table
		 * @param	NetworkVersion the version of this network profiler, determines if we need to do extra parsing with the string
		 * @return	Ip string associated with address table index
		 */
		public string GetIpString( int ConnectionIndex, uint NetworkVersion )
		{
			if( NetworkVersion < 12 )
			{
				UInt64 Addr = AddressArray[ConnectionIndex];
				UInt32 IP = (UInt32)(Addr >> 32);
				UInt32 Port = (UInt32)(Addr & (((UInt64)1 << 32) - 1));

				byte ip0 = (byte)((IP >> 24) & 255);
				byte ip1 = (byte)((IP >> 16) & 255);
				byte ip2 = (byte)((IP >> 8) & 255);
				byte ip3 = (byte)((IP >> 0) & 255);
				return string.Format("{0}.{1}.{2}.{3}: {4}", ip0, ip1, ip2, ip3, Port);
			}
			else
			{
				return StringAddressArray[ConnectionIndex];
			}
		}

		/**
		 * Returns the class name index for the passed in actor name index
		 * 
		 * @param	ActorNameIndex	Name table entry of actor
		 * @return	Class name table index of actor's class
		 */
		public int GetClassNameIndex(int ActorNameIndex)
		{
			
            int ClassNameIndex = 0;
            try
            {
                // Class name is actor name with the trailing _XXX cut off.
                string ActorName = GetName(ActorNameIndex);
                string ClassName = ActorName;
                

                int CharIndex = ActorName.LastIndexOf('_');
                if (CharIndex >= 0)
                {
                    ClassName = ActorName.Substring(0, CharIndex);
                }

                

                // Find class name index in name array.
                
                if (ClassNameToNameIndex.ContainsKey(ClassName))
                {
                    // Found.
                    ClassNameIndex = ClassNameToNameIndex[ClassName];
                }
                // Not found, add to name array and then also dictionary.
                else
                {
                    ClassNameIndex = NameArray.Count;
                    NameArray.Add(ClassName);
                    ClassNameToNameIndex.Add(ClassName, ClassNameIndex);
                }
            }
            catch (System.Exception e)
            {
                System.Console.WriteLine("Error Parsing ClassName for Actor: " + ActorNameIndex + e.ToString());
            }

			return ClassNameIndex;
		}

		/**
		 * Returns the class name index for the passed in actor name
		 * 
		 * @param	ClassName	Name table entry of actor
		 * @return	Class name table index of actor's class
		 */
		public int GetIndexFromClassName( string ClassName )
		{
			if ( ClassNameToNameIndex.ContainsKey( ClassName ) )
			{
				return ClassNameToNameIndex[ClassName];
			}

			return -1;
		}

		/**
		 * Updates the passed in summary dictionary with information of new event.
		 * 
		 * @param	Summaries	Summaries dictionary to update (usually ref to ones contained in this class)
		 * @param	NameIndex	Index of object in name table (e.g. property, actor, RPC)
		 * @param	SizeBits	Size in bits associated with object occurence
		 */
		public void UpdateSummary( ref Dictionary<int,TypeSummary> Summaries, int NameIndex, int SizeBits, float TimeInMS )
		{
			if( Summaries.ContainsKey( NameIndex ) )
			{
				var Summary = Summaries[NameIndex];
				Summary.Count++;
				Summary.SizeBits += SizeBits;
                Summary.TimeInMS += TimeInMS;
			}
			else
			{
				Summaries.Add( NameIndex, new TypeSummary( SizeBits, TimeInMS ) );
			}
		}

		public UInt32 GetVersion()
		{
			return Header.Version;
		}

		public string GetChannelTypeName(int ChannelTypeIndex)
		{
			UInt32 Version = GetVersion();

			if (Version < 11)
			{
				return Enum.GetName(typeof(EChannelTypes), ChannelTypeIndex);
			}
			else
			{
				return GetName(ChannelTypeIndex);
			}
		}
	}

	/** Type agnostic summary for property & actor replication and RPCs. */
	public class TypeSummary
	{
		/** Number of times property was replicated or RPC was called, ... */ 
		public long Count = 1;
		/** Total size in bits. */
		public long SizeBits;
		/** Total ms */
		public float TimeInMS;

		/** Constructor */
		public TypeSummary( long InSizeBits, float InTimeInMS )
		{
			SizeBits = InSizeBits;
            TimeInMS = InTimeInMS;
		}
	}

	public class ObjectReplicationSummary
	{
		public ObjectReplicationSummary(int InObjectNameIndex, List<int> PropertyNameIndices)
		{
			ObjectNameIndex = InObjectNameIndex;
			PropertiesPrivate = new List<PropertyReplicationSummary>();
			for (int i = 0; i < PropertyNameIndices.Count; i++)
			{
				PropertiesPrivate.Add(new PropertyReplicationSummary(PropertyNameIndices[i]));
			}
		}

		public readonly int ObjectNameIndex;
		public int LastReplicatedFrame = -1;
		public int LastReplicateFrameWithData = -1;

		public int NumberOfComparisons = 0;
		public int NumberOfReplications = 0;
		public int NumberOfReplicationsWithData = 0;

		public int NumberOfFramesReplicated = 0;
		public int NumberOfFramesReplicatedWithData = 0;

		public float TimeSpentComparingProperties = 0;

		private List<PropertyReplicationSummary> PropertiesPrivate;
		public ReadOnlyCollection<PropertyReplicationSummary> Properties
		{
			get { return PropertiesPrivate.AsReadOnly(); }
		}
		
	}

	public class PropertyReplicationSummary
	{
		public PropertyReplicationSummary(int InPropertyNameIndex)
		{
			PropertyNameIndex = InPropertyNameIndex;
		}

		public readonly int PropertyNameIndex;
		public int LastReplicatedFrame = -1;
		public int LastComparedFrame = -1;
		public int LastChangedFrame = -1;

		public int NumberOfComparisons = 0;
		public int NumberOfChanges = 0;
		public int NumberOfReplications = 0;

		public int NumberOfFramesReplicated = 0;
		public int NumberOfFramesCompared = 0;
		public int NumberOfFramesChanged = 0;
	}
}
