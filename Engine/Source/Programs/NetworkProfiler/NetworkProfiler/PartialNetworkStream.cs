// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Windows.Forms;

namespace NetworkProfiler
{
	/**
	 * Encapsulates one or more frames worth of tokens together with a summary.
	 */
	public class PartialNetworkStream
	{
		// Time/ frames

		/** Normalized start time of partial stream. */
		public float StartTime = -1;
		/** Normalized end time of partial stream. */
		public float EndTime = 0;
		/** Number of frames this summary covers. */
		public int NumFrames = 0;
		/** Number of events in this frame. */
		public int NumEvents = 0;

		// Actor/ Property replication
		
		/** Number of actors in this partial stream. 
		*/
		public int ActorCount = 0;

		/** Number of replicated actors in this partial stream. 
		*/
		public int ReplicatedActorCount = 0;

		/** Total actor replication time in ms. */
		public float ActorReplicateTimeInMS = 0.0f;

		/** Number of properties replicated. */
		public int PropertyCount = 0;
		/** Total size of properties replicated. */
		public int ReplicatedSizeBits = 0;
		
		// RPC

		/** Number of RPCs replicated. */
		public int RPCCount = 0;
		/** Total size of RPC replication. */
		public int RPCSizeBits = 0;

		// SendBunch

		/** Number of times SendBunch was called. */
		public int SendBunchCount = 0;
		/** Total size of bytes sent. */
		public int SendBunchSizeBits = 0;
		/** Total size of bunch headers sent. */
		public int SendBunchHeaderSizeBits = 0;
		
		/** Call count per channel type. */
		public Dictionary<int, int> SendBunchCountPerChannel = new Dictionary<int,int>();
		/** Size per channel type */
		public Dictionary<int, int> SendBunchSizeBitsPerChannel = new Dictionary<int,int>();
		/** Size of bunch headers per channel type */
		public Dictionary<int, int> SendBunchHeaderSizeBitsPerChannel = new Dictionary<int,int>();
		/** Size of bunch payloads per channel type */
		public Dictionary<int, int> SendBunchPayloadSizeBitsPerChannel = new Dictionary<int,int>();

		// Low level socket

		/** Number of low level socket sends on "Unreal" socket type. */
		public int UnrealSocketCount = 0;
		/** Total size of bytes sent on "Unreal" socket type. */
		public int UnrealSocketSize = 0;
		/** Number of low level socket sends on non-"Unreal" socket types. */
		public int OtherSocketCount = 0;
		/** Total size of bytes sent on non-"Unreal" socket type. */
		public int OtherSocketSize = 0;

		// Acks
		
		/** Number of acks sent. */
		public int SendAckCount = 0;
		/** Total size of acks sent. */
		public int SendAckSizeBits = 0;

		// Exported GUIDs

		/** Number of GUID export bunches sent. */
		public int ExportBunchCount = 0;
		/** Total size of exported GUIDs sent. */
		public int ExportBunchSizeBits = 0;

		// Must be mapped GUIDs

		/** Number of must be mapped GUIDs. */
		public int MustBeMappedGuidCount = 0;
		/** Total size of exported GUIDs sent. */
		public int MustBeMappedGuidSizeBits = 0;

		// Content blocks

		/** Number of content block headers. */
		public int ContentBlockHeaderCount = 0;
		/** Total size of content block headers. */
		public int ContentBlockHeaderSizeBits = 0;
		/** Number of content block footers. */
		public int ContentBlockFooterCount = 0;
		/** Total size of content block footers. */
		public int ContentBlockFooterSizeBits = 0;

		// Property handles

		/** Number of property handles. */
		public int PropertyHandleCount = 0;
		/** Total size of property handles. */
		public int PropertyHandleSizeBits = 0;

		// Detailed information.

		/** List of all tokens in this substream. */
		List<TokenBase> Tokens = new List<TokenBase>();

		/** List of all all unique actors in this substream. */
		UniqueItemTracker<UniqueActor, TokenReplicateActor> UniqueActors = new UniqueItemTracker<UniqueActor, TokenReplicateActor>();

		// Cached internal data.

		/** Delta time of first frame. Passed to constructor as we can't calculate it. */
		float FirstFrameDeltaTime = 0;
		/** Index of "Unreal" name in network stream name table. */
		int NameIndexUnreal = -1;

		/**
		 * Constructor, initializing this substream based on network tokens. 
		 * 
		 * @param	InTokens			Tokens to build partial stream from
		 * @param	InNameIndexUnreal	Index of "Unreal" name, used as optimization
		 * @param	InDeltaTime			DeltaTime of first frame as we can't calculate it
		 */
		public PartialNetworkStream( List<TokenBase> InTokens, int InNameIndexUnreal, float InDeltaTime )
		{
			NameIndexUnreal = InNameIndexUnreal;
			FirstFrameDeltaTime = InDeltaTime;

			// Populate tokens array, weeding out unwanted ones.
			foreach( TokenBase Token in InTokens )
			{
#if false
				// Don't add tokens that don't have any replicated properties.
				// NOTE - We now allow properties that didn't replicate anything so we can show performance stats
				if ( ( Token.TokenType == ETokenTypes.ReplicateActor ) && ( ( ( TokenReplicateActor )Token ).Properties.Count == 0 ) )
				{
					continue;
				}
#endif
				Tokens.Add( Token );
			}

			CreateSummary( NameIndexUnreal, FirstFrameDeltaTime, new FilterValues() );
		}

		/**
		 * Constructor, duplicating the passed in stream while applying the passed in filters.
		 * 
		 * @param	InStream		Stream to duplicate
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 */
		public PartialNetworkStream( PartialNetworkStream InStream, FilterValues InFilterValues )
		{
			NameIndexUnreal = InStream.NameIndexUnreal;
			FirstFrameDeltaTime = InStream.FirstFrameDeltaTime;

			// Merge tokens from passed in stream based on filter criteria.
			foreach( var Token in InStream.Tokens )
			{
				if ( Token.MatchesFilters( InFilterValues ) )
				{
					Tokens.Add( Token );
				}
			}
			CreateSummary( NameIndexUnreal, FirstFrameDeltaTime, InFilterValues );
		}

		public PartialNetworkStream( MainWindow InMainWindow, List<PartialNetworkStream> InStreams, int InStartFrame, int InEndFrame, int InNameIndexUnreal, FilterValues InFilterValues, float InDeltaTime )
		{
			NameIndexUnreal		= InNameIndexUnreal;
			FirstFrameDeltaTime = InDeltaTime;

			InMainWindow.ShowProgress( true );
			InMainWindow.UpdateProgress( 0 );

			// Merge tokens from passed in streams.
			for ( int i = InStartFrame; i < InEndFrame; i++ )
			{
				PartialNetworkStream Stream = InStreams[i];

				if ( i % 10 == 0 )
				{
					float Percent = (float)( i - InStartFrame ) / (float)( InEndFrame - InStartFrame );
					InMainWindow.UpdateProgress( ( int )( Percent * 100 ) );
				}

				// Merge tokens from passed in stream based on filter criteria.
				foreach ( var Token in Stream.Tokens )
				{
					if ( Token.MatchesFilters( InFilterValues ) )
					{
						Tokens.Add( Token );
						UpdateSummary( Token, InFilterValues );
					}
				}
			}

			InMainWindow.ShowProgress( false );

			EndTime += InDeltaTime;
		}

		public PartialNetworkStream( int InNameIndexUnreal, float InDeltaTime )
		{
			NameIndexUnreal = InNameIndexUnreal;
			FirstFrameDeltaTime = InDeltaTime;
		}

		public void AddStream( PartialNetworkStream InStream )
		{
			// Merge tokens from passed in stream based on filter criteria.
			foreach ( var Token in InStream.Tokens )
			{
				Tokens.Add( Token );
				UpdateSummary( Token, new FilterValues() );
			}
		}

		/**
		 * Filters based on the passed in filters and returns a new partial network stream.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 * 
		 * @return	new filtered network stream
		 */
		public PartialNetworkStream Filter( FilterValues InFilterValues )
		{
			return new PartialNetworkStream( this, InFilterValues );
		}

		protected void UpdateSummary( TokenBase Token, FilterValues InFilterValues )
		{
			switch( Token.TokenType )
			{
				case ETokenTypes.FrameMarker:
					var TokenFrameMarker = (TokenFrameMarker) Token;
					if( StartTime < 0 )
					{
						StartTime = TokenFrameMarker.RelativeTime;
						EndTime = TokenFrameMarker.RelativeTime;
					}
					else
					{
						EndTime = TokenFrameMarker.RelativeTime;
					}
					NumFrames++;
					break;
				case ETokenTypes.SocketSendTo:
					var TokenSocketSendTo = (TokenSocketSendTo) Token;
					// Unreal game socket
					if( TokenSocketSendTo.SocketNameIndex == NameIndexUnreal )
					{
						UnrealSocketCount++;
						UnrealSocketSize += TokenSocketSendTo.BytesSent;
					}
					else
					{
						OtherSocketCount++;
						OtherSocketSize += TokenSocketSendTo.BytesSent;
					}
					break;
				case ETokenTypes.SendBunch:
					var TokenSendBunch = (TokenSendBunch) Token;
					SendBunchCount++;
					SendBunchSizeBits += TokenSendBunch.GetNumTotalBits();
					SendBunchHeaderSizeBits += TokenSendBunch.NumHeaderBits;

					int ChannelTypeIndex = TokenSendBunch.GetChannelTypeIndex();

					if (SendBunchCountPerChannel.ContainsKey(ChannelTypeIndex))
					{
						SendBunchCountPerChannel[ChannelTypeIndex]++;
						SendBunchSizeBitsPerChannel[ChannelTypeIndex] += TokenSendBunch.GetNumTotalBits();
						SendBunchHeaderSizeBitsPerChannel[ChannelTypeIndex] += TokenSendBunch.NumHeaderBits;
						SendBunchPayloadSizeBitsPerChannel[ChannelTypeIndex] += TokenSendBunch.NumPayloadBits;
					}
					else
					{
						SendBunchCountPerChannel.Add(ChannelTypeIndex, 1);
						SendBunchSizeBitsPerChannel.Add(ChannelTypeIndex, TokenSendBunch.GetNumTotalBits());
						SendBunchHeaderSizeBitsPerChannel.Add(ChannelTypeIndex, TokenSendBunch.NumHeaderBits);
						SendBunchPayloadSizeBitsPerChannel.Add(ChannelTypeIndex, TokenSendBunch.NumPayloadBits);
					}

					break;
				case ETokenTypes.SendRPC:
					var TokenSendRPC = (TokenSendRPC) Token;
					RPCCount++;
					RPCSizeBits += TokenSendRPC.GetNumTotalBits();
					break;
				case ETokenTypes.ReplicateActor:
					var TokenReplicateActor = (TokenReplicateActor) Token;
					ActorCount++;

					if ( TokenReplicateActor.Properties.Count > 0 )
					{
						ReplicatedActorCount++;
					}

                    ActorReplicateTimeInMS += TokenReplicateActor.TimeInMS;

					foreach( var Property in TokenReplicateActor.Properties )
					{
						if( Property.MatchesFilters( InFilterValues ) )
						{
							PropertyCount++;
							ReplicatedSizeBits += Property.NumBits;
						}
					}

					foreach( var PropertyHeader in TokenReplicateActor.PropertyHeaders )
					{
						if( PropertyHeader.MatchesFilters( InFilterValues ) )
						{
							ReplicatedSizeBits += PropertyHeader.NumBits;
						}
					}

					UniqueActors.AddItem( TokenReplicateActor, TokenReplicateActor.GetClassNameIndex() );

					break;
				case ETokenTypes.Event:
					NumEvents++;
					break;
				case ETokenTypes.RawSocketData:
					break;
				case ETokenTypes.SendAck:
					var TokenSendAck = (TokenSendAck) Token;
					SendAckCount++;
					SendAckSizeBits += TokenSendAck.NumBits;
					break;
				case ETokenTypes.ExportBunch:
					var TokenExportBunch = (TokenExportBunch) Token;
					ExportBunchCount++;
					ExportBunchSizeBits += TokenExportBunch.NumBits;
					break;
				case ETokenTypes.MustBeMappedGuids:
					var TokenMustBeMappedGuids = (TokenMustBeMappedGuids) Token;
					MustBeMappedGuidCount += TokenMustBeMappedGuids.NumGuids;
					MustBeMappedGuidSizeBits += TokenMustBeMappedGuids.NumBits;
					break;
				case ETokenTypes.BeginContentBlock:
					var TokenBeginContentBlock = (TokenBeginContentBlock) Token;
					ContentBlockHeaderCount++;
					ContentBlockHeaderSizeBits += TokenBeginContentBlock.NumBits;
					break;
				case ETokenTypes.EndContentBlock:
					var TokenEndContentBlock = (TokenEndContentBlock) Token;
					ContentBlockFooterCount++;
					ContentBlockFooterSizeBits += TokenEndContentBlock.NumBits;
					break;
				case ETokenTypes.WritePropertyHandle:
					var TokenWritePropertyHandle = (TokenWritePropertyHandle) Token;
					PropertyHandleCount++;
					PropertyHandleSizeBits += TokenWritePropertyHandle.NumBits;
					break;
				default:
					throw new System.IO.InvalidDataException();
			}		
		}

		/**
		 * Parses tokens to create summary.	
		 */
		protected void CreateSummary( int NameIndexUnreal, float DeltaTime, FilterValues InFilterValues )
		{
			foreach( TokenBase Token in Tokens )
			{
				UpdateSummary( Token, InFilterValues );
			}

			EndTime += DeltaTime;
		}

		public void ToDetailedTreeView( TreeNodeCollection Nodes, FilterValues InFilterValues )
		{
			Nodes.Clear();

			foreach ( TokenBase Token in Tokens )
			{
				Token.ToDetailedTreeView( Nodes, InFilterValues );
			}
		}

		private TreeNode AddNode( TreeNodeCollection Nodes, int Index, String Value )
		{
			if ( Nodes.Count <= Index )
			{
				return Nodes.Add( Value );
			}

			Nodes[Index].Text = Value;

			return Nodes[Index];
		}

		public void ToActorSummaryView( NetworkStream NetworkStream, TreeView TreeView )
        {
			bool FirstTree = TreeView.Nodes.Count == 0;

			TreeNode Parent = AddNode( TreeView.Nodes, 0, "Summary" );

			AddNode( Parent.Nodes, 0, "Frames             :" + ConvertToCountString( NumFrames, false ) );
			AddNode( Parent.Nodes, 1, "Seconds            :" + ConvertToCountString( EndTime - StartTime, false ) );

			for ( int i = 0; i < 2; i++ )
			{
				bool bPerSecond = i == 0 ? true : false;

				float OneOverDeltaTime = i == 1 ? 1.0f : 1 / ( EndTime - StartTime );

				// New parent
				Parent = AddNode( TreeView.Nodes, i + 1, i == 1 ? "Details (TOTAL)" : "Details (PER SECOND)" );

				TreeNode Child = null;

				AddNode( Parent.Nodes, 0, "Outgoing bandwidth :" + ConvertToSizeString( ( UnrealSocketSize + OtherSocketSize + NetworkStream.PacketOverhead * ( UnrealSocketCount + OtherSocketCount ) ) * OneOverDeltaTime, bPerSecond ) );
				AddNode( Parent.Nodes, 1, "SocketSend Count   :" + ConvertToCountString( UnrealSocketCount * OneOverDeltaTime, bPerSecond ) );
				AddNode( Parent.Nodes, 2, "SocketSend Size    :" + ConvertToSizeString( UnrealSocketSize * OneOverDeltaTime, bPerSecond ) );

				Child = AddNode( Parent.Nodes, 3, "SendBunchCount     :" + ConvertToCountString( SendBunchCount * OneOverDeltaTime, bPerSecond ) );
				Child.Nodes.Clear();

				int ChildIndex = 0;
				foreach(KeyValuePair<int, int> Pair in SendBunchCountPerChannel)
				{
					AddNode(Child.Nodes, ChildIndex++, NetworkStream.GetChannelTypeName(Pair.Key).PadRight(16) + ":" + ConvertToCountString(Pair.Value * OneOverDeltaTime, bPerSecond));					
				}

				Child = AddNode( Parent.Nodes, 4, "SendBunchSize      :" + ConvertToSizeString( SendBunchSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );
				Child.Nodes.Clear();

				ChildIndex = 0;
				foreach(KeyValuePair<int, int> Pair in SendBunchSizeBitsPerChannel)			
				{	
					AddNode(Child.Nodes, ChildIndex++, NetworkStream.GetChannelTypeName(Pair.Key).PadRight(16) + ":" + ConvertToCountString(Pair.Value  / 8.0f * OneOverDeltaTime, bPerSecond));
				}

				AddNode( Parent.Nodes, 5, "Actor Count        :" + ConvertToCountString( ActorCount * OneOverDeltaTime, bPerSecond ) );
				AddNode( Parent.Nodes, 6, "Replicated Actors  :" + ConvertToCountString( ReplicatedActorCount * OneOverDeltaTime, bPerSecond ) );
				AddNode( Parent.Nodes, 7, "Property Count     :" + ConvertToCountString( PropertyCount * OneOverDeltaTime, bPerSecond ) );
				AddNode( Parent.Nodes, 8, "Property Size      :" + ConvertToSizeString( ReplicatedSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );

				AddNode( Parent.Nodes, 9, "RPC Count          :" + ConvertToCountString( RPCCount, bPerSecond ) );
				AddNode( Parent.Nodes, 10, "RPC Size           :" + ConvertToSizeString( RPCSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );

				AddNode( Parent.Nodes, 11, "Ack Count          :" + ConvertToCountString( SendAckCount, bPerSecond ) );
				AddNode( Parent.Nodes, 12, "Ack Size           :" + ConvertToSizeString( SendAckSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );

				Child = AddNode( Parent.Nodes, 13, "Bunch Overhead     :" + ConvertToSizeString( GetTotalBunchOverheadSizeBits() / 8.0f * OneOverDeltaTime, bPerSecond ) );

				AddNode( Child.Nodes, 0, "Bunch Headers        :" + ConvertToSizeString( SendBunchHeaderSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );
				AddNode( Child.Nodes, 1, "ContentBlock Headers :" + ConvertToSizeString( ContentBlockHeaderSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );
				AddNode( Child.Nodes, 2, "ContentBlock Footers :" + ConvertToSizeString( ContentBlockFooterSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );
				AddNode( Child.Nodes, 3, "Handles              :" + ConvertToSizeString( PropertyHandleSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );
				AddNode( Child.Nodes, 4, "Export GUIDS         :" + ConvertToSizeString( ExportBunchSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );
				AddNode( Child.Nodes, 5, "MustBeMapped GUIDS   :" + ConvertToSizeString( MustBeMappedGuidSizeBits / 8.0f * OneOverDeltaTime, bPerSecond ) );
			}

			// First time through, expand default nodes we want to show
			if ( FirstTree )
			{
				TreeView.Nodes[0].Expand();
				TreeView.Nodes[1].Expand();
			}
		}

		/**
		 * Dumps actor tokens into a list view for viewing performance/bandwidth
		 */
		public void ToActorPerformanceView( NetworkStream NetworkStream, ListView ListView, ListView PropertyListView, FilterValues InFilterValues )
        {
			var ActorDetailList = UniqueActors.UniqueItems.OrderByDescending(s => s.Value.TimeInMS).ToList();

			bool bFirstView = ListView.Items.Count == 0;

			ListView.Items.Clear();

			float OneOverDeltaTime = 1 / ( EndTime - StartTime );

			float TotalTimeInMS = 0;
			foreach (var UniqueActor in ActorDetailList)
			{
				TotalTimeInMS += UniqueActor.Value.TimeInMS;
			}
			float OneOverTotalTimeMS = 1 / TotalTimeInMS;

			var Columns = new string[9];
			foreach (var UniqueActor in ActorDetailList)
			{
				long NumActorBytes = ( UniqueActor.Value.SizeBits + 7 ) / 8;

				Columns[0] = NetworkStream.GetName( UniqueActor.Key );
				Columns[1] = UniqueActor.Value.TimeInMS.ToString( "0.00" );
				Columns[2] = ( UniqueActor.Value.TimeInMS * OneOverTotalTimeMS * 100 ).ToString("0.00");
				Columns[3] = ( NumActorBytes * OneOverDeltaTime / 1024.0f ).ToString( "0.00" );
				Columns[4] = NumActorBytes.ToString();
				Columns[5] = UniqueActor.Value.Count.ToString();
                Columns[6] = ( UniqueActor.Value.Count * OneOverDeltaTime ).ToString( "0.00" );
                Columns[7] = ( UniqueActor.Value.ReplicatedCount * OneOverDeltaTime ).ToString( "0.00" );

				Columns[8] = UniqueActor.Value.Count > 0 ? ( 100.0f - ( ( float )UniqueActor.Value.ReplicatedCount / ( float )UniqueActor.Value.Count * 100.0f ) ).ToString( "0.00" ) : "0";

                ListView.Items.Add( new ListViewItem( Columns ) );
			}

			if ( bFirstView )
			{
				ListView.AutoResizeColumns( ColumnHeaderAutoResizeStyle.ColumnContent );
				ListView.AutoResizeColumns( ColumnHeaderAutoResizeStyle.HeaderSize );
			}

			/*
			if ( ListView.Items.Count > 0 )
			{
				ToPropertyPerformanceView( NetworkStream, ListView.Items[0].Text, PropertyListView, InFilterValues );
			}
			*/
		}

		/**
		 * Dumps property tokens into a list view for viewing performance/bandwidth
		 */
		public void ToPropertyPerformanceView( NetworkStream NetworkStream, string ActorClassName, ListView ListView, FilterValues InFilterValues )
		{
			ListView.Items.Clear();

			UniqueActor Actor;

			int Index = NetworkStream.GetIndexFromClassName( ActorClassName );

			if ( Index != -1 && UniqueActors.UniqueItems.TryGetValue( Index, out Actor ) )
			{
				var Columns = new string[3];

				foreach ( var Property in Actor.Properties.UniqueItems )
				{
					Columns[0] = NetworkStream.GetName( Property.Key );
					Columns[1] = ( ( Property.Value.SizeBits + 7 ) / 8 ).ToString();
					Columns[2] = Property.Value.Count.ToString();

					ListView.Items.Add( new ListViewItem( Columns ) );
				}
			}
		}

		/**
		 * Converts the passed in number of bytes to a string formatted as Bytes, KByte, MByte depending
		 * on magnitude.
		 * 
		 * @param	SizeInBytes		Size in bytes to conver to formatted string
		 * 
		 * @return string representation of value, either Bytes, KByte or MByte
		 */ 
		public string ConvertToSizeString( float SizeInBytes, bool bPerSecond )
		{
			// Format as MByte if size > 1 MByte.
			if( SizeInBytes > 1024 * 1024 )
			{
				string Extra = bPerSecond ? " MB/s" : " MBytes";
				return (SizeInBytes / 1024 / 1024).ToString("0.0").PadLeft(8) + Extra;
			}
			// Format as KByte if size is > 1 KByte and <= 1 MByte
			else if( SizeInBytes > 1024 )
			{
				string Extra = bPerSecond ? " KB/s" : " KBytes";
				return ( SizeInBytes / 1024 ).ToString( "0.0" ).PadLeft( 8 ) + Extra;
			}
			// Format as Byte if size is <= 1 KByte
			else
			{
				string Extra = bPerSecond ? " Bytes/s" : " Bytes";
				return SizeInBytes.ToString( "0.0" ).PadLeft( 8 ) + Extra;
			}
		}

		/**
		 * Converts passed in value to string with appropriate formatting and padding
		 * 
		 * @param	Count	Value to convert
		 * @return	string reprentation with sufficient padding
		 */
		public string ConvertToCountString( float Count, bool bPerSecond )
		{
			string Extra = bPerSecond ? " hz" : "";
			return Count.ToString( "0.0" ).PadLeft( 8 ) + Extra;
		}
		
		/**
		 * Returns the total number of bits used for bunch format protocol information.
		 * This includes bunch headers, content block headers and footers, property handles,
		 * exported GUIDs, and "must be mapped" GUIDs.
		 */
		public int GetTotalBunchOverheadSizeBits()
		{
			return	SendBunchHeaderSizeBits +
					ContentBlockHeaderSizeBits +
					ContentBlockFooterSizeBits +
					PropertyHandleSizeBits +
					ExportBunchSizeBits +
					MustBeMappedGuidSizeBits;
		}
	}

	/**
	 * UniqueItem
	 * Abstract base class for items stored in a UniqueItemTracker class
	 * Holds a unique item, and tracks references to the non unique items that are of the same type
	*/
	class UniqueItem<T> where T : class
	{
		public int	Count		= 0;		// Number of non unique items
		public T	FirstItem	= null;

		public virtual void OnItemAdded( T Item )
		{
			if ( FirstItem == null )
			{
				FirstItem = Item;
			}
			Count++;
		}
	}

	/**
	 * UniqueItemTracker
	 * Helps consolidate a list of items into a unique list
	 * UniqueItems track reference to the non unique items of the same type
	*/
	class UniqueItemTracker<UniqueT, ItemT>
		where ItemT : class
		where UniqueT : UniqueItem<ItemT>, new()
	{
		public Dictionary<int, UniqueT> UniqueItems = new Dictionary<int, UniqueT>();

		public void AddItem(ItemT Item, int Key)
		{
			if ( !UniqueItems.ContainsKey(Key) )
			{
				UniqueItems.Add(Key, new UniqueT());
			}

			UniqueItems[Key].OnItemAdded( Item );
		}
	}

	/**
	 * UniqueProperty
	 * Tracks all the unique properties of the same type
	*/
	class UniqueProperty : UniqueItem<TokenReplicateProperty>
	{
		public long		SizeBits = 0;

		override public void OnItemAdded(TokenReplicateProperty Item)
		{
			base.OnItemAdded(Item);
			SizeBits += Item.NumBits;
		}
	}

	/**
	 * UniquePropertyHeader
	 * Tracks all the unique property headers of the same type
	*/
	class UniquePropertyHeader : UniqueItem<TokenWritePropertyHeader>
	{
		public long		SizeBits = 0;

		override public void OnItemAdded(TokenWritePropertyHeader Item)
		{
			base.OnItemAdded(Item);
			SizeBits += Item.NumBits;
		}
	}

	/**
	 * UniqueActor
	 * Tracks all the unique actors of the same type
	*/
	class UniqueActor : UniqueItem<TokenReplicateActor>
    {
		public long		SizeBits = 0;
        public float	TimeInMS = 0;
		public int		ReplicatedCount = 0;

		public UniqueItemTracker<UniqueProperty, TokenReplicateProperty> Properties = new UniqueItemTracker<UniqueProperty, TokenReplicateProperty>();

		override public void OnItemAdded(TokenReplicateActor Item)
		{
			base.OnItemAdded(Item);

			if ( Item.Properties.Count > 0 )
			{
				ReplicatedCount++;
			}

			TimeInMS += Item.TimeInMS;

			foreach (var Property in Item.Properties)
			{
				SizeBits += Property.NumBits;
				Properties.AddItem(Property, Property.PropertyNameIndex);
			}

			foreach (var PropertyHeader in Item.PropertyHeaders)
			{
				SizeBits += PropertyHeader.NumBits;
			}
		}
	}
}
