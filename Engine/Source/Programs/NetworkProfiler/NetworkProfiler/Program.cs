// Copyright Epic Games, Inc. All Rights Reserved.



using System;
using System.Collections.Generic;
using System.Linq;
using System.Windows.Forms;

namespace NetworkProfiler
{
	static class Program
	{
		/// <summary>
		/// The main entry point for the application.
		/// </summary>
		[STAThread]
		static void Main( string[] Args )
		{
			Application.EnableVisualStyles();
			Application.SetCompatibleTextRenderingDefault(false);
			Application.CurrentCulture = System.Globalization.CultureInfo.InvariantCulture;
			Application.Run(new MainWindow(Args));
		}
	}
}
