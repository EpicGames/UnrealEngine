<?xml version='1.0' encoding='utf-8'?>
<SettingsFile xmlns="http://schemas.microsoft.com/VisualStudio/2004/01/settings" CurrentProfile="(Default)" GeneratedClassNamespace="NetworkProfiler.Properties" GeneratedClassName="Settings">
  <Profiles />
  <Settings>
    <Setting Name="NetworkPerfConnectionString" Type="(Connection string)" Scope="Application">
      <DesignTimeValue Profile="(Default)">&lt;?xml version="1.0" encoding="utf-16"?&gt;
&lt;SerializableConnectionString xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xmlns:xsd="http://www.w3.org/2001/XMLSchema"&gt;
  &lt;ConnectionString&gt;Data Source=dev-db;Initial Catalog=NetworkPerf;Integrated Security=True&lt;/ConnectionString&gt;
  &lt;ProviderName&gt;System.Data.SqlClient&lt;/ProviderName&gt;
&lt;/SerializableConnectionString&gt;</DesignTimeValue>
      <Value Profile="(Default)">Data Source=dev-db;Initial Catalog=NetworkPerf;Integrated Security=True</Value>
    </Setting>
  </Settings>
</SettingsFile>