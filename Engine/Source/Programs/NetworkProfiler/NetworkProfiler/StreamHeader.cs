// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.IO;
using System.Drawing;
using System.Windows.Forms;

namespace NetworkProfiler
{
	/**
	 * Header written by capture tool
	 */
	public class StreamHeader
	{
		/** Magic number at beginning of data file.					*/
		public const UInt32 ExpectedMagic = 0x1DBF348C;

		/** We expect this version, or we can't proceed.			*/
		public const UInt32 MinSupportedVersion = 10;

		/** Magic to ensure we're opening the right file.			*/
		public UInt32 Magic;
		/** Version number to detect version mismatches.			*/
		public UInt32 Version;

		/** Tag, set via -networkprofiler=TAG						*/
		public string Tag;
		/** Game name, e.g. Example									*/
		public string GameName;
		/** URL used to open/ browse to the map.					*/
		public string URL;

		public static void ShowDialog( string Header, string Message )
		{
			Form Dialog = new Form();

			Dialog.Width			= 400;
			Dialog.Height			= 120;
			Dialog.Text				= Header;
			Dialog.FormBorderStyle	= FormBorderStyle.FixedDialog;

			Label LabelMessage = new Label() 
			{ 
				//Left		= 0, 
				//Top		= 0, 
				Width		= Dialog.Width,
				Height		= Dialog.Height / 2,
				Text		= Message, 
				TextAlign	= ContentAlignment.MiddleCenter,
				Font		= new Font( SystemFonts.MessageBoxFont.FontFamily.Name, 12.0f, FontStyle.Bold, GraphicsUnit.Point ) 
			};

			LabelMessage.TextAlign = ContentAlignment.MiddleCenter;

			Button ButtonOK = new Button() { Text = "OK", Left = Dialog.Width - 100, Width = 80, Top = Dialog.Height - 70 };

			ButtonOK.Click += ( sender, e ) => { Dialog.Close(); };

			Dialog.Controls.Add( ButtonOK );
			Dialog.Controls.Add( LabelMessage );
			Dialog.ShowDialog();
		}

		/**
		 * Reads the header information from the passed in stream and returns it. It also returns
		 * a BinaryReader that is endian-appropriate for the data stream. 
		 *
		 * @param	ParserStream		source stream of data to read from
		 * @param	BinaryStream [out]	binary reader used for reading from stream
		 * @return	serialized header
		 */
		public static StreamHeader ReadHeader( Stream ParserStream, out BinaryReader BinaryStream  )
		{
			// Create a binary stream for data, we might toss this later for are an endian swapping one.
			BinaryStream = new BinaryReader(ParserStream,System.Text.Encoding.ASCII);

			// Serialize header.
			StreamHeader Header = new StreamHeader(BinaryStream);

			// Determine whether read file has magic header. If no, try again byteswapped.
			if(Header.Magic != StreamHeader.ExpectedMagic)
			{
				// Seek back to beginning of stream before we retry.
				ParserStream.Seek(0,SeekOrigin.Begin);

				// Use big endian reader. It transparently endian swaps data on read.
				BinaryStream = new BinaryReaderBigEndian(ParserStream);
				
				// Serialize header a second time.
				Header = new StreamHeader(BinaryStream);
			}

			// At this point we should have a valid header. If no, throw an exception.
			if( Header.Magic != StreamHeader.ExpectedMagic )
			{
				ShowDialog( "Error", "Unrecognized file" );
				throw new InvalidDataException();
			}

			if ( Header.Version < StreamHeader.MinSupportedVersion )
			{
				ShowDialog( "Error", "Invalid version" );
				throw new InvalidDataException();
			}

			return Header;
		}

		/**
		 * Constructor, serializing header from passed in stream.
		 * 
		 * @param	BinaryStream	Stream to serialize header from.
		 */
		public StreamHeader(BinaryReader BinaryStream)
		{
			// Serialize the file format magic first.
			Magic = BinaryStream.ReadUInt32();

			// Stop serializing data if magic number doesn't match. Most likely endian issue.
			if ( Magic != ExpectedMagic )
			{
				return;
			}

			// Version info for backward compatible serialization.
			Version = BinaryStream.ReadUInt32();

			// Stop serializing data if version number doesn't match
			if ( Version < MinSupportedVersion )
			{
				return;
			}

			// Serialize various dynamically-sized strings.
			Tag = SerializeAnsiString( BinaryStream );
			GameName = SerializeAnsiString( BinaryStream );
			URL = SerializeAnsiString( BinaryStream );
		}

		/**
		 * Serialize a string from the header.
		 * 
		 * @param	BinaryStream	stream to read from
		 * @return	string being read
		 */
		private string SerializeAnsiString( BinaryReader BinaryStream )
		{
			UInt32 SerializedLength = BinaryStream.ReadUInt32();
			return new string(BinaryStream.ReadChars((int)SerializedLength));
		}
	}
}
