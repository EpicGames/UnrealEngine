// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.Linq;
using System.Text;
using System.IO;
using System.Windows.Forms;
using System.Diagnostics;

namespace NetworkProfiler
{
	class StreamParser
	{
		public static NetworkStream NetworkStream = new NetworkStream();

		/**
		 * Helper function for handling updating actor summaries as they require a bit more work.
		 * 
		 * @param	NetworkStream			NetworkStream associated with token
		 * @param	TokenReplicateActor		Actor token
		 */
		private static void HandleActorSummary(NetworkStream NetworkStream, TokenReplicateActor TokenReplicateActor )
		{
			if ( TokenReplicateActor != null )
			{
				int ClassNameIndex = NetworkStream.GetClassNameIndex(TokenReplicateActor.ActorNameIndex);
                NetworkStream.UpdateSummary(ref NetworkStream.ActorNameToSummary, ClassNameIndex, TokenReplicateActor.GetNumReplicatedBits( new FilterValues() ), TokenReplicateActor.TimeInMS );
			}
		}

		/**
		 * Helper function for handling housekeeping that needs to happen when we parse a new actor
		 * We used to emit actors before properties, but now we emit properties before actors
		 * So when we are about to parse a new actor, we need to copy the properties up to that point to this new actor
		 * 
		 * @param	TokenReplicateActor		Actor token
		 * @param	LastProperties			Properties to be copied to the actor
		 */
		private static void FinishActorProperties(TokenReplicateActor TokenReplicateActor, List<TokenReplicateProperty> LastProperties, List<TokenWritePropertyHeader> LastPropertyHeaders)
		{
			for (int i = 0; i < LastProperties.Count; i++ )
			{
				TokenReplicateActor.Properties.Add(LastProperties[i]);
			}
			LastProperties.Clear();

			for (int i = 0; i < LastPropertyHeaders.Count; i++ )
			{
				TokenReplicateActor.PropertyHeaders.Add(LastPropertyHeaders[i]);
			}
			LastPropertyHeaders.Clear();
		}

		/**
		 * Parses passed in data stream into a network stream container class
		 * 
		 * @param	ParserStream	Raw data stream, needs to support seeking
		 * @return	NetworkStream data was parsed into
		 */
		public static NetworkStream Parse( MainWindow InMainWindow, Stream ParserStream )
		{
            var StartTime = DateTime.UtcNow;

			// Network stream the file is parsed into.
			NetworkStream = new NetworkStream();
			
			// Serialize the header. This will also return an endian-appropriate binary reader to
			// be used for reading the data. 
			BinaryReader BinaryStream = null; 
			NetworkStream.Header = StreamHeader.ReadHeader( ParserStream, out BinaryStream );

			// Scratch variables used for building stream. Required as we emit information in reverse
			// order needed for parsing.
			var CurrentFrameTokens = new List<TokenBase>();
			TokenReplicateActor LastActorToken = null;
            List<TokenReplicateProperty> LastProperties = new List<TokenReplicateProperty>();
			List<TokenWritePropertyHeader> LastPropertyHeaders = new List<TokenWritePropertyHeader>();

			Dictionary<int, TokenPropertyComparison> ObjectNamesToPropertyComparisons = new Dictionary<int, TokenPropertyComparison>();

			TokenFrameMarker LastFrameMarker = null;

			InMainWindow.ShowProgress( true );

			int Count = 0;

			var AllFrames = new PartialNetworkStream( NetworkStream.NameIndexUnreal, 1.0f / 30.0f );

			int EarlyOutMinutes = InMainWindow.GetMaxProfileMinutes();

			// Parse stream till we reach the end, marked by special token.
			bool bHasReachedEndOfStream = false;

			List<TokenBase> TokenList = new List<TokenBase>();

			float FrameStartTime = -1.0f;
			float FrameEndTime = -1.0f;

			while( bHasReachedEndOfStream == false )
			{
				if ( Count++ % 1000 == 0 )
				{
					float Percent = ( float )ParserStream.Position / ( float )ParserStream.Length;
					InMainWindow.UpdateProgress( ( int )( Percent * 100 ) );
				}

				if ( ParserStream.Position == ParserStream.Length )
				{
					// We reached stream early (must not have been finalized properly, but we can still read it)
					break;
				}

				TokenBase Token = null;

				try
				{
					Token = TokenBase.ReadNextToken( BinaryStream, NetworkStream );
				}
				catch ( System.IO.EndOfStreamException )
				{
					// We reached stream early (must not have been finalized properly, but we can still read it)
					break;
				}

				if ( Token.TokenType == ETokenTypes.NameReference )
				{
					NetworkStream.NameArray.Add( ( Token as TokenNameReference ).Name );

					// Find "Unreal" name index used for misc socket parsing optimizations.
					if ( NetworkStream.NameArray[NetworkStream.NameArray.Count - 1] == "Unreal" )
					{
						NetworkStream.NameIndexUnreal = NetworkStream.NameArray.Count - 1;
					}
					continue;
				}

				if ( Token.TokenType == ETokenTypes.ConnectionReference )
				{
					if( NetworkStream.GetVersion() < 12 )
					{
						NetworkStream.AddressArray.Add((Token as TokenConnectionReference).Address);
					}
					else
					{
						NetworkStream.StringAddressArray.Add((Token as TokenConnectionStringReference).Address);
					}
					
					continue;
				}

				if ( Token.TokenType == ETokenTypes.ConnectionChange )
				{
					// We need to setup CurrentConnectionIndex, since it's used in ReadNextToken
					NetworkStream.CurrentConnectionIndex = ( Token as TokenConnectionChanged ).AddressIndex;
					continue;
				}

				TokenList.Add( Token );

				// Track frame start/end times manually so we can bail out when we hit the amount of time we want to load
				if ( Token.TokenType == ETokenTypes.FrameMarker )
				{
					var TokenFrameMarker = ( TokenFrameMarker )Token;

					if ( FrameStartTime < 0 )
					{
						FrameStartTime = TokenFrameMarker.RelativeTime;
						FrameEndTime = TokenFrameMarker.RelativeTime;
					}
					else
					{
						FrameEndTime = TokenFrameMarker.RelativeTime;
					}
				}

				if ( EarlyOutMinutes > 0 && ( ( FrameEndTime - FrameStartTime ) > 60 * EarlyOutMinutes ) )
				{
					break;
				}
			}

			for ( int i = 0; i < TokenList.Count; i++ )
			{
				if ( i % 1000 == 0 )
				{
					float Percent = ( float )( i + 1 ) / ( float )( TokenList.Count );
					InMainWindow.UpdateProgress( ( int )( Percent * 100 ) );
				}

				TokenBase Token = TokenList[i];

				// Convert current tokens to frame if we reach a frame boundary or the end of the stream.
				if( ((Token.TokenType == ETokenTypes.FrameMarker) || (Token.TokenType == ETokenTypes.EndOfStreamMarker))
				// Nothing to do if we don't have any tokens, e.g. first frame.
				&&	(CurrentFrameTokens.Count > 0) )				
				{
					// Figure out delta time of previous frame. Needed as partial network stream lacks relative
					// information for last frame. We assume 30Hz for last frame and for the first frame in case
					// we receive network traffic before the first frame marker.
					float DeltaTime = 1 / 30.0f;
					if( Token.TokenType == ETokenTypes.FrameMarker && LastFrameMarker != null )
					{
						DeltaTime = ((TokenFrameMarker) Token).RelativeTime - LastFrameMarker.RelativeTime;
					}

					// Create per frame partial stream and add it to the full stream.
					var FrameStream = new PartialNetworkStream( CurrentFrameTokens, NetworkStream.NameIndexUnreal, DeltaTime );
					
					AllFrames.AddStream( FrameStream );

					NetworkStream.Frames.Add(FrameStream);
					CurrentFrameTokens.Clear();

					Debug.Assert(LastProperties.Count == 0);		// We shouldn't have any properties now
					Debug.Assert(LastPropertyHeaders.Count == 0);	// We shouldn't have any property headers now either

					// Finish up actor summary of last pending actor before switching frames.
					HandleActorSummary(NetworkStream, LastActorToken);
					LastActorToken = null;
				}
				// Keep track of last frame marker.
				if( Token.TokenType == ETokenTypes.FrameMarker )
				{
					LastFrameMarker = (TokenFrameMarker) Token;
					ObjectNamesToPropertyComparisons = new Dictionary<int, TokenPropertyComparison>();

					//Console.Out.WriteLine("EndOfFrame: " + NetworkStream.Frames.Count.ToString("0"));
				}

				// Bail out if we hit the end. We already flushed tokens above.
				if( Token.TokenType == ETokenTypes.EndOfStreamMarker )
				{
					Debug.Assert(LastProperties.Count == 0);		// We shouldn't have any properties now
					Debug.Assert(LastPropertyHeaders.Count == 0);	// We shouldn't have any property headers now either
					bHasReachedEndOfStream = true;
					// Finish up actor summary of last pending actor at end of stream
					HandleActorSummary(NetworkStream, LastActorToken);
				}
				// Keep track of per frame tokens.
				else
				{
					// Keep track of last actor context for property replication.
					if( Token.TokenType == ETokenTypes.ReplicateActor )
					{
						// Encountered a new actor so we can finish up existing one for summary.
						FinishActorProperties(Token as TokenReplicateActor, LastProperties, LastPropertyHeaders);
						Debug.Assert(LastProperties.Count == 0);		// We shouldn't have any properties now
						Debug.Assert(LastPropertyHeaders.Count == 0);	// We shouldn't have any property headers now either
						HandleActorSummary(NetworkStream, LastActorToken);
						LastActorToken = Token as TokenReplicateActor;
					}
					// Keep track of RPC summary
					else if( Token.TokenType == ETokenTypes.SendRPC )
					{
						var TokenSendRPC = Token as TokenSendRPC;
						NetworkStream.UpdateSummary( ref NetworkStream.RPCNameToSummary, TokenSendRPC.FunctionNameIndex, TokenSendRPC.GetNumTotalBits(), 0.0f );
					}

					// Add properties to the actor token instead of network stream and keep track of summary.
					if( Token.TokenType == ETokenTypes.ReplicateProperty )
					{
						var TokenReplicateProperty = Token as TokenReplicateProperty;
						NetworkStream.UpdateSummary(ref NetworkStream.PropertyNameToSummary, TokenReplicateProperty.PropertyNameIndex, TokenReplicateProperty.NumBits, 0 );
						//LastActorToken.Properties.Add(TokenReplicateProperty);
                        LastProperties.Add(TokenReplicateProperty);
					}
					else if( Token.TokenType == ETokenTypes.WritePropertyHeader )
					{
						var TokenWritePropertyHeader = Token as TokenWritePropertyHeader;
                        LastPropertyHeaders.Add(TokenWritePropertyHeader);
					}
					else if ( Token.TokenType == ETokenTypes.PropertyComparison )
					{
						var TokenPropertyComparison = Token as TokenPropertyComparison;
						ObjectNamesToPropertyComparisons[TokenPropertyComparison.ObjectNameIndex] = TokenPropertyComparison;
						HandleObjectComparison(NetworkStream, TokenPropertyComparison);
					}
					else if ( Token.TokenType == ETokenTypes.ReplicatePropertiesMetaData )
					{
						var TokenReplicatePropertiesMetaData = Token as TokenReplicatePropertiesMetaData;
						TokenPropertyComparison Comparison = null;
						if (ObjectNamesToPropertyComparisons.TryGetValue(TokenReplicatePropertiesMetaData.ObjectNameIndex, out Comparison))
						{
							HandleObjectReplication(NetworkStream, Comparison, TokenReplicatePropertiesMetaData);
						}
						else
						{
							Console.Out.WriteLine(NetworkStream.GetName(TokenReplicatePropertiesMetaData.ObjectNameIndex));
						}
					}
					else
					{
						CurrentFrameTokens.Add(Token);
					}
				}
			}

			InMainWindow.SetCurrentStreamSelection( NetworkStream, AllFrames, false );

			InMainWindow.ShowProgress( false );

			// Stats for profiling.
            double ParseTime = (DateTime.UtcNow - StartTime).TotalSeconds;
			Console.WriteLine( "Parsing {0} MBytes in stream took {1} seconds", ParserStream.Length / 1024 / 1024, ParseTime );

			// Empty stream will have 0 frames and proper name table. Shouldn't happen as we only
			// write out stream in engine if there are any events.
			return NetworkStream;
		}

		/**
		 * Parses summaries into a list view using the network stream for name lookup.
		 * 
		 * @param	NetworkStream	Network stream used for name lookup
		 * @param	Summaries		Summaries to parse into listview
		 * @param	ListView		List view to parse data into
		 */
		public static void ParseStreamIntoListView( NetworkStream NetworkStream, Dictionary<int,TypeSummary> Summaries, ListView ListView )
		{
			ListView.BeginUpdate();
			ListView.Items.Clear();

			// Columns are total size KByte, count, avg size in bytes, avg size in bits and associated name.
			var Columns = new string[7];
			foreach( var SummaryEntry in Summaries )
			{
				Columns[0] = ((float)SummaryEntry.Value.SizeBits / 8 / 1024).ToString("0.0");
				Columns[1] = SummaryEntry.Value.Count.ToString();
                Columns[2] = ((float)SummaryEntry.Value.SizeBits / 8 / SummaryEntry.Value.Count).ToString("0.0");
				Columns[3] = ((float)SummaryEntry.Value.SizeBits / SummaryEntry.Value.Count).ToString("0.0");
                Columns[4] = SummaryEntry.Value.TimeInMS.ToString("0.00");
				Columns[5] = (SummaryEntry.Value.TimeInMS / SummaryEntry.Value.Count).ToString("0.0000");
				Columns[6] = NetworkStream.GetName(SummaryEntry.Key);
                ListView.Items.Add(new ListViewItem(Columns));
			}

			ListView.AutoResizeColumns(ColumnHeaderAutoResizeStyle.HeaderSize);
			ListView.EndUpdate();
		}

		private static void HandleObjectComparison(NetworkStream NetworkStream, TokenPropertyComparison ObjectComparison)
		{
			// We're guaranteed that before any Replication takes place (at least in a single frame), a comparison will also take place.
			// So, if this is the very first comparison for a given object class, we'll also need to set up our summary.

			ObjectReplicationSummary ObjectSummary = null;
			if (!NetworkStream.ObjectNameToReplicationSummary.TryGetValue(ObjectComparison.ObjectNameIndex, out ObjectSummary))
			{
				// If we don't have a comparison token the first time we see this object, something is broken in the stream.
				Debug.Assert(ObjectComparison.ExportedPropertyNames != null);

				ObjectSummary = new ObjectReplicationSummary(ObjectComparison.ObjectNameIndex, ObjectComparison.ExportedPropertyNames);
				NetworkStream.ObjectNameToReplicationSummary.Add(ObjectSummary.ObjectNameIndex, ObjectSummary);
			}

			ObjectSummary.NumberOfComparisons++;
			ObjectSummary.TimeSpentComparingProperties += ObjectComparison.TimeSpentComparing;

			ReadOnlyCollection<PropertyReplicationSummary> ObjectProperties = ObjectSummary.Properties;
			BitArray PropertiesCompared = ObjectComparison.ComparedProperties;
			BitArray PropertiesChanged = ObjectComparison.ChangedProperties;

			Debug.Assert(PropertiesChanged.Count == ObjectProperties.Count);
			Debug.Assert(PropertiesCompared.Count == ObjectProperties.Count);

			int FrameNum = NetworkStream.Frames.Count;

			for (int i = 0; i < ObjectProperties.Count; i++)
			{
				if (PropertiesCompared[i])
				{
					PropertyReplicationSummary ObjectProperty = ObjectProperties[i];
					ObjectProperty.NumberOfComparisons++;

					if (ObjectProperty.LastComparedFrame != FrameNum)
					{
						ObjectProperty.LastComparedFrame++;
						ObjectProperty.NumberOfFramesCompared++;
					}

					if (PropertiesChanged[i])
					{
						ObjectProperty.NumberOfChanges++;
						if (ObjectProperty.LastChangedFrame != FrameNum)
						{
							ObjectProperty.LastChangedFrame = FrameNum;
							ObjectProperty.NumberOfFramesChanged++;
						}
					}
				}
			}
		}

		private static void HandleObjectReplication(NetworkStream NetworkStream, TokenPropertyComparison ObjectComparison, TokenReplicatePropertiesMetaData ObjectReplication)
		{
			// TODO: We may be able to move this data into the Per Frame data so we can display it when selecting a range.
			//			For now, we're just going to show a summary for the entire profile.

			
			// If we're replicating this object, we're guaranteed that it was compared before
			// so this should be valid.
			ObjectReplicationSummary ObjectSummary = NetworkStream.ObjectNameToReplicationSummary[ObjectReplication.ObjectNameIndex];

			int FrameNum = NetworkStream.Frames.Count;

			ReadOnlyCollection<PropertyReplicationSummary> ObjectProperties = ObjectSummary.Properties;
			BitArray PropertiesCompared = ObjectComparison.ComparedProperties;
			BitArray PropertiesChanged = ObjectComparison.ChangedProperties;
			BitArray PropertiesFiltered = ObjectReplication.FilteredProperties;

			// At this point, our object summary should be up to date so we can move on to property summaries.
			// We will do that by comparing the bitfields sent 
			Debug.Assert(PropertiesChanged.Count == ObjectProperties.Count);
			Debug.Assert(PropertiesCompared.Count == ObjectProperties.Count);
			Debug.Assert(PropertiesFiltered.Count == ObjectProperties.Count);

			ObjectSummary.NumberOfReplications++;
			if (ObjectSummary.LastReplicatedFrame != FrameNum)
			{
				// If this is the first replication for an object on a given frame, we better have compared its properties.
				ObjectSummary.LastReplicatedFrame = FrameNum;
				ObjectSummary.NumberOfFramesReplicated++;
			}

			for (int i = 0; i < ObjectProperties.Count; i++)
			{
				if (PropertiesCompared[i] && PropertiesChanged[i] && !PropertiesFiltered[i])
				{
					PropertyReplicationSummary ObjectProperty = ObjectProperties[i];

					// The property was compared, changed, and wasn't filtered, that means it was replicated.
					// Note, we ignore the WasNewObjectComparison here because we may legitimately replicate this
					// property multiple times in the same frame to multiple connections and individual connections
					// can have different filters applied to them.
					ObjectProperty.NumberOfReplications++;
					if (ObjectProperty.LastReplicatedFrame != FrameNum)
					{
						ObjectProperty.LastReplicatedFrame = FrameNum;
						ObjectProperty.NumberOfFramesReplicated++;
					}
				}
			}
		}

		public static void ParseStreamIntoReplicationListView( NetworkStream NetworkStream, Dictionary<int, ObjectReplicationSummary> Summaries, ListView ListView )
		{
			ListView.BeginUpdate();
			ListView.Items.Clear();

			// Columns are "Object Class", "# Comparisons", "# Replications", "Comparison Time", "Avg. Time Per Compare"
			var Columns = new string[5];
			foreach (var SummaryEntry in Summaries)
			{
				Columns[0] = NetworkStream.GetName(SummaryEntry.Key);
				Columns[1] = (SummaryEntry.Value.NumberOfComparisons).ToString("0");
				Columns[2] = (SummaryEntry.Value.NumberOfReplications).ToString("0");
				Columns[3] = (SummaryEntry.Value.TimeSpentComparingProperties).ToString("0.0");
				Columns[4] = ((float)SummaryEntry.Value.TimeSpentComparingProperties / (float)SummaryEntry.Value.NumberOfComparisons).ToString("0.000");
				ListView.Items.Add(new ListViewItem(Columns));
			}

			ListView.AutoResizeColumns(ColumnHeaderAutoResizeStyle.HeaderSize);
			ListView.EndUpdate();
		}

		public static void ParseStreamIntoPropertyReplicationListView(NetworkStream NetworkStream, ReadOnlyCollection<PropertyReplicationSummary> Summaries, ListView ListView )
		{
			ListView.BeginUpdate();
			ListView.Items.Clear();

			if (Summaries != null)
			{
				// Columns are "Property", "# Comparisons", "# Times Changed", "# Replications"
				var Columns = new string[4];
				foreach (var Summary in Summaries)
				{
					Columns[0] = NetworkStream.GetName(Summary.PropertyNameIndex);
					Columns[1] = (Summary.NumberOfComparisons).ToString("0");
					Columns[2] = (Summary.NumberOfChanges).ToString("0");
					Columns[3] = (Summary.NumberOfReplications).ToString("0");
					ListView.Items.Add(new ListViewItem(Columns));
				}
			}

			ListView.AutoResizeColumns(ColumnHeaderAutoResizeStyle.HeaderSize);
			ListView.EndUpdate();
		}
	}
}
