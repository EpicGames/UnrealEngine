// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Windows.Forms;

namespace NetworkProfiler
{
	/** Enum values need to be in sync with UE3 */
	public enum ETokenTypes
	{
		FrameMarker			= 0,		// Frame marker, signaling beginning of frame.	
		SocketSendTo,					// FSocket::SendTo
		SendBunch,						// UChannel::SendBunch
		SendRPC,						// Sending RPC
		ReplicateActor,					// Replicated object	
		ReplicateProperty,				// Property being replicated.
		EndOfStreamMarker,				// End of stream marker		
		Event,							// Event
		RawSocketData,					// Raw socket data being sent
		SendAck,						// Ack being sent
		WritePropertyHeader,			// Property header being written
		ExportBunch,					// Exported GUIDs
		MustBeMappedGuids,				// Must be mapped GUIDs
		BeginContentBlock,				// Content block headers
		EndContentBlock,				// Content block footers
		WritePropertyHandle,			// Property handles
		ConnectionChange,				// Connection changed
		NameReference,					// Reference to name
		ConnectionReference,			// Reference to connection
		PropertyComparison,				// Data about property comparions.
		ReplicatePropertiesMetaData,	// Data about properties that were filtered out during replication.
		MaxAndInvalid,					// Invalid token, also used as the max token index
	}

	/** Enum values need to be in sync with UE3 */
	public enum EChannelTypes
	{
		Invalid				= 0,	// Invalid type.
		Control,					// Connection control.
		Actor,						// Actor-update channel.
		File,						// Binary file transfer.
		Voice,						// Voice channel
		Max,
	}

	/**
	 * Base class of network token/ events
	 */
	public class TokenBase
	{
		/** Type of token. */
		public ETokenTypes TokenType = ETokenTypes.MaxAndInvalid;

		/** Connection this token belongs to */
		public int ConnectionIndex = 0;

		/** Stats about token types being serialized. */
		public static int[] TokenTypeStats = Enumerable.Repeat(0, (int)ETokenTypes.MaxAndInvalid).ToArray();

		/**
		 * Reads the next token from the stream and returns it.
		 * 
		 * @param	BinaryStream	Stream used to serialize from
		 * @param	InNetworkStream	Network stream this token belongs to
		 * @return	Token serialized
		 */
		public static TokenBase ReadNextToken(BinaryReader BinaryStream, NetworkStream InNetworkStream)
		{
			TokenBase SerializedToken = null;

			ETokenTypes TokenType = (ETokenTypes) BinaryStream.ReadByte();
			// Handle token specific serialization.
			switch( TokenType )
			{
				case ETokenTypes.FrameMarker:
					SerializedToken = new TokenFrameMarker( BinaryStream );
					break;
				case ETokenTypes.SocketSendTo:
					SerializedToken = new TokenSocketSendTo( BinaryStream );
					break;
				case ETokenTypes.SendBunch:
					SerializedToken = new TokenSendBunch( BinaryStream, InNetworkStream.GetVersion() );
					break;
				case ETokenTypes.SendRPC:
					SerializedToken = new TokenSendRPC( BinaryStream, InNetworkStream.GetVersion() );
					break;
				case ETokenTypes.ReplicateActor:
					SerializedToken = new TokenReplicateActor( BinaryStream );
					break;
				case ETokenTypes.ReplicateProperty:
					SerializedToken = new TokenReplicateProperty( BinaryStream );
					break;
				case ETokenTypes.EndOfStreamMarker:
					SerializedToken = new TokenEndOfStreamMarker();
					break;
				case ETokenTypes.Event:
					SerializedToken = new TokenEvent( BinaryStream );
					break;
				case ETokenTypes.RawSocketData:
					SerializedToken = new TokenRawSocketData( BinaryStream );
					break;
				case ETokenTypes.SendAck:
					SerializedToken = new TokenSendAck( BinaryStream );
					break;
				case ETokenTypes.WritePropertyHeader:
					SerializedToken = new TokenWritePropertyHeader( BinaryStream );
					break;
				case ETokenTypes.ExportBunch:
					SerializedToken = new TokenExportBunch( BinaryStream );
					break;
				case ETokenTypes.MustBeMappedGuids:
					SerializedToken = new TokenMustBeMappedGuids( BinaryStream );
					break;
				case ETokenTypes.BeginContentBlock:
					SerializedToken = new TokenBeginContentBlock( BinaryStream );
					break;
				case ETokenTypes.EndContentBlock:
					SerializedToken = new TokenEndContentBlock( BinaryStream );
					break;
				case ETokenTypes.WritePropertyHandle:
					SerializedToken = new TokenWritePropertyHandle( BinaryStream );
					break;
				case ETokenTypes.NameReference:
					SerializedToken = new TokenNameReference( BinaryStream );
					break;
				case ETokenTypes.ConnectionReference:
					{
						if (InNetworkStream.GetVersion() < 12)
						{
							SerializedToken = new TokenConnectionReference(BinaryStream);
						}
						else
						{
							SerializedToken = new TokenConnectionStringReference(BinaryStream);
						}
					}
					break;
				case ETokenTypes.ConnectionChange:
					SerializedToken = new TokenConnectionChanged( BinaryStream );
					break;

				case ETokenTypes.PropertyComparison:
					SerializedToken = new TokenPropertyComparison(BinaryStream);
					break;

				case ETokenTypes.ReplicatePropertiesMetaData:
					SerializedToken = new TokenReplicatePropertiesMetaData(BinaryStream);
					break;

				default:
					throw new InvalidDataException();
			}

			TokenTypeStats[(int)TokenType]++;
			SerializedToken.TokenType = TokenType;

			SerializedToken.ConnectionIndex = InNetworkStream.CurrentConnectionIndex;
			return SerializedToken;
		}

		public virtual void ToDetailedTreeView( TreeNodeCollection List, FilterValues InFilterValues )
		{

		}

		/**
		 * Returns whether the token matches/ passes based on the passed in filters.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 */
		public virtual bool MatchesFilters( FilterValues InFilterValues )
		{
			if ( TokenType == ETokenTypes.FrameMarker || TokenType == ETokenTypes.EndOfStreamMarker )
			{
				return true;
			}

			return InFilterValues.ConnectionMask == null || InFilterValues.ConnectionMask.Contains( ConnectionIndex );
		}
	}

	/**
	 * End of stream token.
	 */
	class TokenEndOfStreamMarker : TokenBase
	{
	}

	/**
	 * Frame marker token.
	 */ 
	class TokenFrameMarker : TokenBase
	{
		/** Relative time of frame since start of engine. */
		public float RelativeTime;

		/** Constructor, serializing members from passed in stream. */
		public TokenFrameMarker(BinaryReader BinaryStream)
		{
			RelativeTime = BinaryStream.ReadSingle();
		}

		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "Frame Markers" );

			Child.Nodes.Add( "Absolute time : " + RelativeTime );
		}
	}

	/**
	 * FSocket::SendTo token. A special address of 0.0.0.0 is used for ::Send
	 */ 
	class TokenSocketSendTo : TokenBase
	{
		/** Socket debug description name index. "Unreal" is special name for game traffic. */
		public int SocketNameIndex;
		/** Bytes actually sent by low level code. */
		public UInt16 BytesSent;
		/** Number of bits representing the packet id */
		public UInt16 NumPacketIdBits;
		/** Number of bits representing bunches */
		public UInt16 NumBunchBits;
		/** Number of bits representing acks */
		public UInt16 NumAckBits;
		/** Number of bits used for padding */
		public UInt16 NumPaddingBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenSocketSendTo(BinaryReader BinaryStream)
		{
			SocketNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
			BytesSent = BinaryStream.ReadUInt16();
			NumPacketIdBits = BinaryStream.ReadUInt16();
			NumBunchBits = BinaryStream.ReadUInt16();
			NumAckBits = BinaryStream.ReadUInt16();
			NumPaddingBits = BinaryStream.ReadUInt16();
		}

		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "Socket SendTo" );

			Child = Child.Nodes.Add( "Destination : " + StreamParser.NetworkStream.GetIpString( ConnectionIndex, StreamParser.NetworkStream.GetVersion() ) );

			Child.Nodes.Add( "SocketName          : " + StreamParser.NetworkStream.GetName( SocketNameIndex ) );
			Child.Nodes.Add( "DesiredBytesSent    : " + ( NumPacketIdBits + NumBunchBits + NumAckBits + NumPaddingBits ) / 8.0f );
			Child.Nodes.Add( "   NumPacketIdBits  : " + NumPacketIdBits );
			Child.Nodes.Add( "   NumBunchBits     : " + NumBunchBits );
			Child.Nodes.Add( "   NumAckBits       : " + NumAckBits );
			Child.Nodes.Add( "   NumPaddingBits   : " + NumPaddingBits );
			Child.Nodes.Add( "BytesSent           : " + BytesSent );
		}

		public override bool MatchesFilters( FilterValues InFilterValues )
		{
			return base.MatchesFilters( InFilterValues );
		}
	}

	/**
	 * UChannel::SendBunch token, NOTE that this is NOT SendRawBunch	
	 */
	class TokenSendBunch : TokenBase
	{
		/** Channel index. */
		public UInt16 ChannelIndex;
		/** Channel type. */
		protected byte ChannelType;
		/** Channel type name index. */
		protected int ChannelTypeNameIndex;
		/** Number of header bits serialized/sent. */
		public UInt16 NumHeaderBits;
		/** Number of non-header bits serialized/sent. */
		public UInt16 NumPayloadBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenSendBunch(BinaryReader BinaryStream, UInt32 Version)
		{
			ChannelIndex = BinaryStream.ReadUInt16();
			if (Version < 11)
			{
				ChannelType = BinaryStream.ReadByte();
				ChannelTypeNameIndex = -1;
			}
			else
			{
				ChannelType = 0;
				ChannelTypeNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
			}

			NumHeaderBits = BinaryStream.ReadUInt16();
			NumPayloadBits = BinaryStream.ReadUInt16();
		}

		public int GetChannelTypeIndex()
		{
			if (ChannelTypeNameIndex != -1)
			{
				return ChannelTypeNameIndex;
			}
			else
			{
				return ChannelType;
			}
		}

		/**
		 * Gets the total number of bits serialized for the bunch.
		 */
		public int GetNumTotalBits()
		{
			return NumHeaderBits + NumPayloadBits;
		}

		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "Send Bunch" );

			Child = Child.Nodes.Add( "Channel Type  : " + StreamParser.NetworkStream.GetChannelTypeName( GetChannelTypeIndex() ) );
			Child.Nodes.Add( "Channel Index    : " + ChannelIndex );
			Child.Nodes.Add( "NumTotalBits     : " + GetNumTotalBits() );
			Child.Nodes.Add( "   NumHeaderBits : " + NumHeaderBits );
			Child.Nodes.Add( "   NumPayloadBits: " + NumPayloadBits );
			Child.Nodes.Add( "NumTotalBytes    : " + GetNumTotalBits() / 8.0f );
		}
	}

	/**
	 * Token for RPC replication
	 */
	class TokenSendRPC : TokenBase
	{
		/** Name table index of actor name. */
		public int ActorNameIndex;
		/** Name table index of function name. */
		public int FunctionNameIndex;
		/** Number of bits serialized/sent for the header. */
		public int NumHeaderBits;
		/** Number of bits serialized/sent for the parameters. */
		public int NumParameterBits;
		/** Number of bits serialized/sent for the footer. */
		public int NumFooterBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenSendRPC(BinaryReader BinaryStream, UInt32 Version)
		{
			ActorNameIndex = TokenHelper.LoadPackedInt(BinaryStream);
			FunctionNameIndex = TokenHelper.LoadPackedInt(BinaryStream);

			if (Version < 13)
			{
				NumHeaderBits = BinaryStream.ReadUInt16();
				NumParameterBits = BinaryStream.ReadUInt16();
				NumFooterBits = BinaryStream.ReadUInt16();
			}
			else
			{
				NumHeaderBits = TokenHelper.LoadPackedInt(BinaryStream);
				NumParameterBits = TokenHelper.LoadPackedInt(BinaryStream);
				NumFooterBits = TokenHelper.LoadPackedInt(BinaryStream);
			}
		}

		/**
		 * Gets the total number of bits serialized for the RPC.
		 */
		public int GetNumTotalBits()
		{
			return NumHeaderBits + NumParameterBits + NumFooterBits;
		}

		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "RPCs" );

			Child = Child.Nodes.Add(StreamParser.NetworkStream.GetName( FunctionNameIndex ) );

			Child.Nodes.Add( "Actor               : " + StreamParser.NetworkStream.GetName( ActorNameIndex ) );
			Child.Nodes.Add( "NumTotalBits        : " + GetNumTotalBits() );
			Child.Nodes.Add( "   NumHeaderBits    : " + NumHeaderBits );
			Child.Nodes.Add( "   NumParameterBits : " + NumParameterBits );
			Child.Nodes.Add( "   NumFooterBits    : " + NumFooterBits );
			Child.Nodes.Add( "NumTotalBytes       : " + GetNumTotalBits() / 8.0f );
		}

		/**
		 * Returns whether the token matches/ passes based on the passed in filters.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 * 
		 * @return true if it matches, false otherwise
		 */
		public override bool MatchesFilters( FilterValues InFilterValues )
		{
			return base.MatchesFilters( InFilterValues ) && ( InFilterValues.ActorFilter.Length == 0 || StreamParser.NetworkStream.GetName( ActorNameIndex ).ToUpperInvariant().Contains( InFilterValues.ActorFilter.ToUpperInvariant() ) )
			&& ( InFilterValues.RPCFilter.Length == 0 || StreamParser.NetworkStream.GetName( FunctionNameIndex ).ToUpperInvariant().Contains( InFilterValues.RPCFilter.ToUpperInvariant() ) );
		}
	}

	/**
	 * Actor replication token. Like the frame marker, this doesn't actually correlate
	 * with any data transfered but is status information for parsing. Properties are 
	 * removed from stream after parsing and moved into actors.
	 */ 
	class TokenReplicateActor : TokenBase
	{
        public enum ENetFlags
        {
            Dirty = 1,
            Initial = 2,
            Owner = 4
        }
        /** Whether bNetDirty, bnetInitial, or bNetOwner was set on Actor. */
        public byte NetFlags;
		/** Name table index of actor name */
		public int ActorNameIndex;
		/** Time in ms to replicate this actor */
		public float TimeInMS;

        /** List of property tokens that were serialized for this actor. */
		public List<TokenReplicateProperty> Properties;

		/** List of property header tokens that were serialized for this actor. */
		public List<TokenWritePropertyHeader> PropertyHeaders;

		/** Constructor, serializing members from passed in stream. */
		public TokenReplicateActor(BinaryReader BinaryStream)
		{
			NetFlags = BinaryStream.ReadByte();
			ActorNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
            TimeInMS = BinaryStream.ReadSingle();
			Properties = new List<TokenReplicateProperty>();
			PropertyHeaders = new List<TokenWritePropertyHeader>();
		}

		/**
		 * Returns the number of bits for this replicated actor while taking filters into account.
		 * 
		 * @param	ActorFilter		Filter for actor name
		 * @param	PropertyFilter	Filter for property name
		 * @param	RPCFilter		Unused
		 */
		public int GetNumReplicatedBits( FilterValues InFilterValues )
		{
			int NumReplicatedBits = 0;
			foreach( var Property in Properties )
			{
				if( Property.MatchesFilters( InFilterValues ) )
				{
					NumReplicatedBits += Property.NumBits;
				}
			}

			foreach( var PropertyHeader in PropertyHeaders )
			{
				if( PropertyHeader.MatchesFilters( InFilterValues ) )
				{
					NumReplicatedBits += PropertyHeader.NumBits;
				}
			}

			return NumReplicatedBits;
		}
	 
		/**
		 * Fills tree view with description of this token
		 * 
		 * @param	Tree			Tree to fill in 
		 * @param	ActorFilter		Filter for actor name
		 * @param	PropertyFilter	Filter for property name
		 * @param	RPCFilter		Unused
		 */
		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "Replicated Actors" );

			int NumReplicatedBits = GetNumReplicatedBits( InFilterValues );

			string Flags = ((NetFlags & (byte)ENetFlags.Dirty) == 1 ? "bNetDirty " : "" ) + ((NetFlags & (byte)ENetFlags.Initial) == 1 ? "bNetInitial" : "" ) + ((NetFlags & (byte)ENetFlags.Owner) == 1 ? "bNetOwner" : "" );
			Child = Child.Nodes.Add( string.Format( "{0,-32} : {1:0.00} ({2:000}) ", StreamParser.NetworkStream.GetName( ActorNameIndex ), TimeInMS, NumReplicatedBits / 8 ) + Flags );

			if ( Properties.Count > 0 )
			{
				TreeNode NewChild = Child.Nodes.Add( "Properties" );
				foreach ( var Property in Properties )
				{
					if ( Property.MatchesFilters( InFilterValues ) )
					{
						NewChild.Nodes.Add( string.Format( "{0,-25} : {1:000}", StreamParser.NetworkStream.GetName( Property.PropertyNameIndex ), Property.NumBits / 8.0f ) );
					}
				}
			}

			if ( PropertyHeaders.Count > 0 )
			{
				TreeNode NewChild = Child.Nodes.Add( "Property Headers" );
				foreach ( var PropertyHeader in PropertyHeaders )
				{
					if ( PropertyHeader.MatchesFilters( InFilterValues ) )
					{
						NewChild.Nodes.Add( string.Format( "{0,-25} : {1:000}", StreamParser.NetworkStream.GetName( PropertyHeader.PropertyNameIndex ), PropertyHeader.NumBits / 8.0f ) );
					}
				}
			}
		}

		/**
		 * Returns whether the token matches/ passes based on the passed in filters.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 * 
		 * @return true if it matches, false otherwise
		 */
		public override bool MatchesFilters( FilterValues InFilterValues )
		{
			bool ContainsMatchingProperty = false || ( Properties.Count == 0 && InFilterValues.PropertyFilter.Length == 0 );
			foreach( var Property in Properties )
			{
				if( Property.MatchesFilters( InFilterValues ) )
				{
					ContainsMatchingProperty = true;
					break;
				}
			}
			return base.MatchesFilters( InFilterValues ) && ( InFilterValues.ActorFilter.Length == 0 || StreamParser.NetworkStream.GetName( ActorNameIndex ).ToUpperInvariant().Contains( InFilterValues.ActorFilter.ToUpperInvariant() ) ) && ContainsMatchingProperty;
		}

		public int GetClassNameIndex()
		{
			return StreamParser.NetworkStream.GetClassNameIndex( ActorNameIndex );
		}	
	}

	/**
	 * Token for property replication. Context determines which actor this belongs to.
	 */
	class TokenReplicateProperty : TokenBase
	{
		/** Name table index of property name. */
		public int		PropertyNameIndex;
		/** Number of bits serialized/ sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenReplicateProperty(BinaryReader BinaryStream)
		{
			PropertyNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
			NumBits = BinaryStream.ReadUInt16();
		}

		/**
		 * Returns whether the token matches/ passes based on the passed in filters.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 * 
		 * @return true if it matches, false otherwise
		 */
		public override bool MatchesFilters( FilterValues InFilterValues )
		{
			return base.MatchesFilters( InFilterValues ) && ( InFilterValues.PropertyFilter.Length == 0 || StreamParser.NetworkStream.GetName( PropertyNameIndex ).ToUpperInvariant().Contains( InFilterValues.PropertyFilter.ToUpperInvariant() ) );
		}
	}			

	/**
	 * Token for property header replication. Context determines which actor this belongs to.
	 */
	class TokenWritePropertyHeader : TokenBase
	{
		/** Name table index of property name. */
		public int		PropertyNameIndex;
		/** Number of bits serialized/ sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenWritePropertyHeader(BinaryReader BinaryStream)
		{
			PropertyNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
			NumBits = BinaryStream.ReadUInt16();
		}

		/**
		 * Returns whether the token matches/ passes based on the passed in filters.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 * 
		 * @return true if it matches, false otherwise
		 */
		public override bool MatchesFilters( FilterValues InFilterValues )
		{
			return base.MatchesFilters( InFilterValues ) && ( InFilterValues.PropertyFilter.Length == 0 || StreamParser.NetworkStream.GetName( PropertyNameIndex ).ToUpperInvariant().Contains( InFilterValues.PropertyFilter.ToUpperInvariant() ) );
		}
	}

	/**
	 * Token for exported GUID bunches.
	 */
	class TokenExportBunch : TokenBase
	{
		/** Number of bits serialized/ sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenExportBunch(BinaryReader BinaryStream)
		{
			NumBits = BinaryStream.ReadUInt16();
		}

		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "GUID's" );

			Child.Nodes.Add( "NumBytes         : " + NumBits / 8.0f );
		}
	}

	/**
	 * Token for must be mapped GUIDs.
	 */
	class TokenMustBeMappedGuids : TokenBase
	{
		/** Number of GUIDs serialized/sent. */
		public UInt16	NumGuids;

		/** Number of bits serialized/sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenMustBeMappedGuids(BinaryReader BinaryStream)
		{
			NumGuids = BinaryStream.ReadUInt16();
			NumBits = BinaryStream.ReadUInt16();
		}

		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "Must Be Mapped GUID's" );

			Child.Nodes.Add( "NumGuids         : " + NumGuids );
			Child.Nodes.Add( "NumBytes         : " + NumBits / 8.0f );
		}
	}

	/**
	 * Token for content block headers.
	 */
	class TokenBeginContentBlock : TokenBase
	{
		/** Name table index of property name. */
		public int		ObjectNameIndex;
		/** Number of bits serialized/ sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenBeginContentBlock(BinaryReader BinaryStream)
		{
			ObjectNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
			NumBits = BinaryStream.ReadUInt16();
		}

		/**
		 * Returns whether the token matches/ passes based on the passed in filters.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 * 
		 * @return true if it matches, false otherwise
		 */
		public override bool MatchesFilters( FilterValues InFilterValues )
		{
			return base.MatchesFilters( InFilterValues ) && ( InFilterValues.ActorFilter.Length == 0 || StreamParser.NetworkStream.GetName( ObjectNameIndex ).ToUpperInvariant().Contains( InFilterValues.ActorFilter.ToUpperInvariant() ) );
		}
	}

	/**
	 * Token for property header replication. Context determines which actor this belongs to.
	 */
	class TokenEndContentBlock : TokenBase
	{
		/** Name table index of property name. */
		public int		ObjectNameIndex;
		/** Number of bits serialized/ sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenEndContentBlock(BinaryReader BinaryStream)
		{
			ObjectNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
			NumBits = BinaryStream.ReadUInt16();
		}

		/**
		 * Returns whether the token matches/ passes based on the passed in filters.
		 * 
		 * @param	ActorFilter		Actor filter to match against
		 * @param	PropertyFilter	Property filter to match against
		 * @param	RPCFilter		RPC filter to match against
		 * 
		 * @return true if it matches, false otherwise
		 */
		public override bool MatchesFilters( FilterValues InFilterValues )
		{
			return base.MatchesFilters( InFilterValues ) && ( InFilterValues.ActorFilter.Length == 0 || StreamParser.NetworkStream.GetName( ObjectNameIndex ).ToUpperInvariant().Contains( InFilterValues.ActorFilter.ToUpperInvariant() ) );
		}
	}

	/**
	 * Token for property handle replication. Context determines which actor this belongs to.
	 */
	class TokenWritePropertyHandle : TokenBase
	{
		/** Number of bits serialized/ sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenWritePropertyHandle(BinaryReader BinaryStream)
		{
			NumBits = BinaryStream.ReadUInt16();
		}
	}

	/**
	 * Token for connection change event
	 */
	class TokenConnectionChanged : TokenBase
	{
		/** Number of bits serialized/ sent. */
		public Int32 AddressIndex;

		/** Constructor, serializing members from passed in stream. */
		public TokenConnectionChanged( BinaryReader BinaryStream )
		{
			AddressIndex = TokenHelper.LoadPackedInt( BinaryStream );
		}
	}

	/**
	 * Token for connection reference event
	 */
	class TokenNameReference : TokenBase
	{
		/** Address of connection */
		public string Name = null;

		/** Constructor, serializing members from passed in stream. */
		public TokenNameReference( BinaryReader BinaryStream )
		{
			UInt32 Length = BinaryStream.ReadUInt32();
			Name = new string(BinaryStream.ReadChars((int)Length));
		}
	}

	/**
	 * Token for connection reference event
	 */
	class TokenConnectionReference : TokenBase
	{
		/** Address of connection */
		public UInt64 Address;

		/** Constructor, serializing members from passed in stream. */
		public TokenConnectionReference( BinaryReader BinaryStream )
		{
			Address = BinaryStream.ReadUInt64();
		}
	}

	/**
	 * Token for connection reference as a string.
	 * This allows for support for different address formats without having to do any additional work.
	 * Addresses are pushed in via the ToString(true) call on an FInternetAddr
	 */
	class TokenConnectionStringReference : TokenBase
	{
		/** Address of connection */
		public string Address = null;

		/** Constructor, serializing members from passed in stream. */
		public TokenConnectionStringReference(BinaryReader BinaryStream)
		{
			int StrLength = Math.Abs(BinaryStream.ReadInt32());
			Address = new string(BinaryStream.ReadChars(StrLength));
		}
	}

	/**
	 * Token for events.
	 */
	class TokenEvent : TokenBase
	{
		/** Name table index of event name. */
		public int EventNameNameIndex;
		/** Name table index of event description. */
		public int EventDescriptionNameIndex;

		/** Constructor, serializing members from passedin stream. */
		public TokenEvent(BinaryReader BinaryStream)
		{
			EventNameNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
			EventDescriptionNameIndex = TokenHelper.LoadPackedInt( BinaryStream );
		}

		/**
		 * Fills tree view with description of this token
		 * 
		 * @param	Tree			Tree to fill in 
		 * @param	ActorFilter		Filter for actor name
		 * @param	PropertyFilter	Filter for property name
		 * @param	RPCFilter		Unused
		 */
		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "Events" );

			Child.Nodes.Add( "Type          : " + StreamParser.NetworkStream.GetName( EventNameNameIndex ) );
			Child.Nodes.Add( "Description   : " + StreamParser.NetworkStream.GetName( EventDescriptionNameIndex ) );
		}
	}

	/**
	 * Token for raw socket data. Not captured by default in UE3.
	 */
	class TokenRawSocketData : TokenBase
	{
		/** Raw data. */
		public byte[] RawData;

		/** Constructor, serializing members from passed in stream. */
		public TokenRawSocketData(BinaryReader BinaryStream)
		{
			int Size = BinaryStream.ReadUInt16();
			RawData = BinaryStream.ReadBytes( Size );
		}
	}

	/**
	 * Token for sent acks.
	 */
	class TokenSendAck : TokenBase
	{
		/** Number of bits serialized/sent. */
		public UInt16	NumBits;

		/** Constructor, serializing members from passed in stream. */
		public TokenSendAck(BinaryReader BinaryStream)
		{
			NumBits = BinaryStream.ReadUInt16();
		}

		public override void ToDetailedTreeView( TreeNodeCollection Tree, FilterValues InFilterValues )
		{
			TreeNode Child = TokenHelper.AddNode( Tree, "Send Acks" );

			Child.Nodes.Add( "NumBytes : " + NumBits / 8.0f );
		}
	}

	/**
	 * Token that tracks information about property comparisons for objects.
	 */ 
	class TokenPropertyComparison : TokenBase
	{
		/** Index to the Name of the object whose properties we were comparing. */
		public int ObjectNameIndex;

		/** The amount of time we spent comparing the properties. */
		public float TimeSpentComparing;

		/**
		 * A BitArray describing which of the top level properties of the object were actually compared.
		 * The number of bits will always match the number of top level properties in the class.
		 */
		public BitArray ComparedProperties;

		/**
		 * A BitArray describing which of the top level properties of the object were found to have changed
		 * after we compared them.
		 * The number of bits will always match the number of top level properties in the class.
		 */ 
		public BitArray ChangedProperties;

		public List<int> ExportedPropertyNames;

		public TokenPropertyComparison(BinaryReader BinaryStream)
		{
			ObjectNameIndex = TokenHelper.LoadPackedInt(BinaryStream);
			TimeSpentComparing = BinaryStream.ReadSingle();
			TokenHelper.ReadBitArray(BinaryStream, ref ComparedProperties);
			TokenHelper.ReadBitArray(BinaryStream, ref ChangedProperties);

			int NumExportedPropertyNames = TokenHelper.LoadPackedInt(BinaryStream);
			if (NumExportedPropertyNames > 0)
			{
				ExportedPropertyNames = new List<int>(NumExportedPropertyNames);
				for (int i = 0; i < NumExportedPropertyNames; ++i)
				{
					ExportedPropertyNames.Add(TokenHelper.LoadPackedInt(BinaryStream));
				}
			}
		}
	}

	/**
	 * Token that tracks basic metadata about replication for objects.
	 */ 
	class TokenReplicatePropertiesMetaData : TokenBase
	{
		/** Index to the Name of the object whose properties we were replicating. */
		public int ObjectNameIndex;

		/**
		 * Whether or not we resent our entire history.
		 * This is used to indicate we were resending everything for replay recording (checkpoints).
		 * Note, properties that were filtered for the connection or that were inactive won't have
		 * been sent, so using FilteredProperties is still required to see what was actually sent.
		 */ 
		public bool bSentAllChangedProperties;

		/**
		 * A BitArray describing which of the top level properties of the object were inactive (would not
		 * be replicated) during a call to ReplicateProperties.
		 * The number of bits will always match the number of top level properties in the class,
		 * unless bWasAnythingSent is false (in which case it will be null).
		 */
		public BitArray FilteredProperties;

		public TokenReplicatePropertiesMetaData(BinaryReader BinaryStream)
		{
			ObjectNameIndex = TokenHelper.LoadPackedInt(BinaryStream);

			byte Flags = BinaryStream.ReadByte();
			bSentAllChangedProperties = (Flags & 0x1) != 0;
			TokenHelper.ReadBitArray(BinaryStream, ref FilteredProperties);
		}
	}


	public class TokenHelper
	{
		public const int NumBitsPerDWord = 32;
		public const int NumBitsPerDWordLog2 = 5;

		static public TreeNode AddNode( TreeNodeCollection Tree, string Text )
		{
			TreeNode[] Childs = Tree.Find( Text, false );

			TreeNode Child = null;

			if ( Childs == null || Childs.Length == 0 )
			{
				Child = Tree.Add( Text );
				Child.Name = Text;
			}
			else
			{
				Child = Childs[0];
			}

			return Child;
		}

		public static int LoadPackedInt( BinaryReader BinaryStream )
		{
			UInt32 Value = 0;
			byte cnt = 0;
			bool more = true;
			while ( more )
			{
				UInt32 NextByte = BinaryStream.ReadByte();

				more = ( NextByte & 1 ) != 0;		// Check 1 bit to see if theres more after this
				NextByte = NextByte >> 1;			// Shift to get actual 7 bit value
				Value += NextByte << ( 7 * cnt++ );	// Add to total value
			}

			return (int)Value;
		}

		public static void ReadBitArray(BinaryReader BinaryStream, ref BitArray OutBitArray)
		{
			// TODO: Verify Endianness
			int NumBits = LoadPackedInt(BinaryStream);
			int NumInts = ((NumBits + NumBitsPerDWord - 1) >> NumBitsPerDWordLog2);
			int[] ReadValues = new int[NumInts];

			for (Int32 Idx = 0; Idx < NumInts; Idx++)
			{
				ReadValues[Idx] = LoadPackedInt(BinaryStream);
			}

			OutBitArray = new BitArray(ReadValues);
			OutBitArray.Length = NumBits;
		}
	}
}