<?xml version="1.0"?>
<configuration>
  <connectionStrings>
    <add name="NetworkProfiler.Properties.Settings.NetworkPerfConnectionString" connectionString="Data Source=dev-db;Initial Catalog=NetworkPerf;Integrated Security=True" providerName="System.Data.SqlClient"/>
  </connectionStrings>
  <runtime>
    <generatePublisherEvidence enabled="false"/>
    <gcAllowVeryLargeObjects enabled="true"/>
  </runtime>
  <startup>
    <supportedRuntime version="v4.0" sku=".NETFramework,Version=v4.6.2"/>
  </startup>
</configuration>
